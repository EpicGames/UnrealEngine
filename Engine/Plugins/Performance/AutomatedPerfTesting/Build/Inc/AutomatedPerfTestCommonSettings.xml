<?xml version="1.0" encoding="utf-8"?>
<BuildGraph xmlns="http://www.epicgames.com/BuildGraph" xmlns:xsi="http://www.w3.org/2001/XMLSchema-instance" xsi:schemaLocation="http://www.epicgames.com/BuildGraph ../../../../../../Engine/Build/Graph/Schema.xsd" >
	
	<!-- Horde Configuration -->
	<EnvVar Name="UE_HORDE_JOBID"/>
	<EnvVar Name="UE_HORDE_STEPID"/>
	<Property Name="JobDetails" Value="$(HordeDashboardURL)/job/$(UE_HORDE_JOBID)?step=$(UE_HORDE_STEPID)"/>
	<Property Name="APTBuildVersion" Value="$(EscapedBranch)-CL-$(Change)-$(ProjectName)"/> 
	<Property Name="APTTestBuildVersion" Value="$(EscapedBranch)-CL-$(Change)-$(ProjectName)$(PreflightSuffix)"/> 
	<!-- Insights Setup for Automated Perf Testing -->
	<!-- since we need to tell UAT whether or not to using the shipping configuration of Insights when processing -->
	<!-- insights traces after the conclusion of the test --> 
	<Option Name="UseShippingInsights" Restrict="true|false" DefaultValue="false" Description="Whether or not to use the shipping configuration of Insights." />

	<!-- then we'll use that to override values that will be picked up by the rest of BuildAndTestProject.xml -->
	<Property Name="IncludeUnrealInsightsInToolCompile" Value="true" />
	<Property Name="UnrealInsightsBuildConfiguration" Value="Development" If="!$(UseShippingInsights)" />
	<Property Name="UnrealInsightsBuildConfiguration" Value="Shipping" If="$(UseShippingInsights)" />
	
	<!-- Other common arguments --> 
	<Option Name="TestID" DefaultValue="APT" Description="A string to use to identify these performance tests" />
	<Option Name="DoCSVProfiler" DefaultValue="true" Restrict="true|false" Description="Whether or not to use CSVProfiler to gather and parse performance telemetry." />
	<Option Name="PerfReportServer" DefaultValue="false" Restrict="true|false" Description="Whether or not to upload CSVProfiler data to PRS." />
	<Option Name="HordeTelemetry" DefaultValue="false" Restrict="true|false" Description="Whether or not to emit CSV performance summary data to Horde." />
	<Option Name="DoFPSChart" DefaultValue="false" Restrict="true|false" Description="Whether or not to use FPS Chart to gather and parse performance telemetry." />
	<Option Name="TraceChannels" DefaultValue="default,screenshot,stats" Description="Which channels to use when tracing with Insights" />
	<Option Name="DoVideoCapture" Restrict="true|false" DefaultValue="false" Description="Whether or not to capture video during the test" />
	<Option Name="DataSourceProjectNameOverride" DefaultValue="" Restrict="^(|[A-Za-z][A-Za-z0-9_]{0,19})$" Description="Override the ProjectName segment of the PRS DataSourceName, which is in the form 'Automation.ProjectName.DataSourceType}'. Values must start with a leter, can use only use alpha numeric characters and _, and must be 20 characters or less. "/>
	<Option Name="GenerateLocalReports" DefaultValue="false" Restrict="true|false" Description="Whether or not to create a local PerfReportTool report" />
	<Option Name="ResX" DefaultValue="3840" Restrict="^[0-9]+$" Description="Resolution width to run tests with" />
	<Option Name="ResY" DefaultValue="2160" Restrict="^[0-9]+$" Description="Resolution height to run tests with" />
	<Option Name="IterationsPerf" DefaultValue="6" Restrict="^[0-9]+$" Description="Number of iterations to run perf test with" />
	<Option Name="IterationsLLM" DefaultValue="3" Restrict="^[0-9]+$" Description="Number of iterations to run LLM test with" />
	<Option Name="IterationsGPUPerf" DefaultValue="3" Restrict="^[0-9]+$" Description="Number of iterations to run GPUPerf test with" />
	<Option Name="IterationsInsights" DefaultValue="1" Restrict="^[0-9]+$" Description="Number of iterations to run Insights test with" />
	<Option Name="IterationsGPUReshape" DefaultValue="1" Restrict="^[0-9]+$" Description="Number of iterations to run GPUReshape test with" />
	<Option Name="UsePlatformExtraDebugMemory" DefaultValue="true" Restrict="true|false" Description="Use additional debug memory on target device if supported. Applies only to LLM and Insights trace."/>
	<Option Name="APTIgnoreTestBuildLogging" Restrict="true|false" DefaultValue="true" Description="If true, tests run in Test build configuration will not fail if logs are empty." />
	<Option Name="PerfOutputPath" DefaultValue="" Description="Path where Perf Test artifacts (Logs, CSVs, Reports, Cache) will be stored. Override to a common network-accessible path when running on build machines if needed"/>
	<Option Name="ProjectPath" DefaultValue="$(DefaultProjectPath)" Restrict=".+" Description="Path to the folder that contains your project"/>
	<Option Name="TestBuildVersion" DefaultValue="$(APTTestBuildVersion)" Description="Overrides test build version"/>
	<Option Name="ExtraAutomatedPerformanceCommonArgs" DefaultValue="" Description="Additional Common arguments passed to Gauntlet command line"/>
	<Option Name="NullOSSPlatforms" DefaultValue="Android" Description=""/>

	<!-- Override build version with given value. If this value is empty, APT will use default build version-->
	<Property Name="BuildVersion" Value="$(APTBuildVersion)"/>
	<Property Name="BuildVersion" Value="$(APTTestBuildVersion)" If="$(IsPreflight)"/>

	<!-- Plugin-provided tests -->
	<Option Name="WithAPTSequenceTests" DefaultValue="true" Restrict="true|false" Description="Whether to run Auto Perf Test sequence flythrough test." />
	<Option Name="WithAPTStaticCameraTests" DefaultValue="false" Restrict="true|false" Description="Whether to run Auto Perf Test static camera tests."  />
	<Option Name="WithAPTMaterialTest" DefaultValue="false" Restrict="true|false" Description="Whether to run Auto Perf Test material test."  />
	<Option Name="WithAPTReplayTest"   DefaultValue="false" Restrict="true|false" Description="Whether to run Auto Perf Test replay test."  />

	<!-- Test sub-types, act as flavors to the above tests -->
	<Option Name="DoPerf" DefaultValue="false" Restrict="true|false" Description="Whether to run Perf (general performance) test." />
	<Option Name="DoLLM" DefaultValue="false" Restrict="true|false" Description="Whether to run LLM (memory) test." />
	<Option Name="DoGPUPerf" DefaultValue="false" Restrict="true|false" Description="Whether to run GPUPerf (GPU performance) test." />
	<Option Name="DoInsightsTrace" DefaultValue="false" Restrict="true|false" Description="Whether or not to capture Unreal Insights trace during the run." />
	<Option Name="DoGPUReshape" DefaultValue="false" Restrict="true|false" Description="Whether to run GPU instrumentation test with GPU Reshape." />
	<Option Name="GPUReshapeWorkspace" DefaultValue="BasicWorkspace" Description="User workspace configuration for GPU Reshape." />

	<!-- Construct the common arguments for the performance test and report commands -->
	<Property Name="AutomatedPerformanceCommonArgs" Value = "-AutomatedPerfTest.UseShippingInsights=$(UseShippingInsights)" />
	<Property Name="AutomatedPerformanceCommonArgs" Value="$(AutomatedPerformanceCommonArgs) -AutomatedPerfTest.TestID=$(TestID)"/>
	<Property Name="AutomatedPerformanceCommonArgs" Value="$(AutomatedPerformanceCommonArgs) -AutomatedPerfTest.TraceChannels=$(TraceChannels)" If="$(DoInsightsTrace) and '$(TraceChannels)' != ''"/>
	<Property Name="AutomatedPerformanceCommonArgs" Value="$(AutomatedPerformanceCommonArgs) -AutomatedPerfTest.DoFPSChart" If="$(DoFPSChart)"/>

	<Property Name="AutomatedPerformanceCommonArgs" Value="$(AutomatedPerformanceCommonArgs) -PerfReportServer" If="$(PerfReportServer) And $(DoCSVProfiler)"/>
	<Property Name="AutomatedPerformanceCommonArgs" Value="$(AutomatedPerformanceCommonArgs) -HordeTelemetry" If="$(HordeTelemetry) And $(DoCSVProfiler)"/>
	<Property Name="AutomatedPerformanceCommonArgs" Value="$(AutomatedPerformanceCommonArgs) -JobDetails=&quot;$(JobDetails)&quot;" If="'$(JobDetails)' != ''"/>
	<Property Name="AutomatedPerformanceCommonArgs" Value="$(AutomatedPerformanceCommonArgs) -NoLocalReports" If="!$(GenerateLocalReports) And $(DoCSVProfiler)" />
	<Property Name="AutomatedPerformanceCommonArgs" Value="$(AutomatedPerformanceCommonArgs) -resX=$(ResX) -resY=$(ResY)"/>
	<Property Name="AutomatedPerformanceCommonArgs" Value="$(AutomatedPerformanceCommonArgs) -AutomatedPerfTest.DoVideoCapture" If="$(DoVideoCapture)"/>
	<Property Name="AutomatedPerformanceCommonArgs" Value="$(AutomatedPerformanceCommonArgs) -AutomatedPerfTest.IgnoreTestBuildLogging" If="$(APTIgnoreTestBuildLogging)"/>
	<Property Name="AutomatedPerformanceCommonArgs" Value="$(AutomatedPerformanceCommonArgs) -AutomatedPerfTest.PerfOutputPath=&quot;$(PerfOutputPath)&quot;" If="'$(PerfOutputPath)' != ''"/>
	<Property Name="AutomatedPerformanceCommonArgs" Value="$(AutomatedPerformanceCommonArgs) $(ExtraAutomatedPerformanceCommonArgs)" If="'$(ExtraAutomatedPerformanceCommonArgs)' != ''"/>
	<Property Name="AutomatedPerformanceCommonArgs" Value="$(AutomatedPerformanceCommonArgs) -Branch=&quot;$(EscapedBranch)&quot; -CL=$(Change)"/>
	<Property Name="AutomatedPerformanceCommonArgs" Value="$(AutomatedPerformanceCommonArgs) -AutomatedPerfTest.TestBuildVersion=&quot;$(TestBuildVersion)&quot;"/>
	<Property Name="AutomatedPerformanceCommonArgs" Value="$(AutomatedPerformanceCommonArgs) -AutomatedPerfTest.BuildVersion=&quot;$(BuildVersion)&quot;"/>
	<Property Name="AutomatedPerformanceCommonArgs" Value="$(AutomatedPerformanceCommonArgs) -AutomatedPerfTest.NullOSSPlatforms=&quot;$(NullOSSPlatforms)&quot;" If="'$(NullOSSPlatforms)' != ''"/>
	<Property Name="AutomatedPerformanceCommonArgs" Value="$(AutomatedPerformanceCommonArgs) -AutomatedPerfTest.DataSourceProjectNameOverride=&quot;$(DataSourceProjectNameOverride)&quot;" If="'$(DataSourceProjectNameOverride)' != ''"/>

	<Property Name="PerfTestCommonArgs"     Value="$(AutomatedPerformanceCommonArgs) -iterations=$(IterationsPerf)"/>
	<Property Name="LLMTestCommonArgs"      Value="$(AutomatedPerformanceCommonArgs) -iterations=$(IterationsLLM)"/>
	<Property Name="GPUPerfTestCommonArgs"  Value="$(AutomatedPerformanceCommonArgs) -iterations=$(IterationsGPUPerf)"/>
	<Property Name="InsightsTestCommonArgs" Value="$(AutomatedPerformanceCommonArgs) -iterations=$(IterationsInsights)"/>
	
	<!-- Reshape configuration -->
	<Do If="$(DoGPUReshape)">
		<Property Name="GPUReshapePath"           Value="$(RootDir)\Engine\Binaries\ThirdParty\GPUReshape\Win64\Raytracing"/>
		<Property Name="GPUReshapeBootstrapPath"  Value="$(RootDir)\Engine\Source\Programs\GPUReshape"/>
		<Property Name="GPUReshapeReportPath"     Value="$(GPUReshapeWorkspace).GRS.Report.json"/>
		<Property Name="GPUReshapeReportPath"     Value="$(PerfOutputPath)\$(GPUReshapeReportPath)" If="'$(PerfOutputPath)' != ''"/>
		<Property Name="GPUReshapeWorkspacePath"  Value="$(GPUReshapeBootstrapPath)\Resources\Workspaces\$(GPUReshapeWorkspace).json"/>
		<Property Name="GPUReshapeSymbolPath"     Value="$(RootDir)\$(ProjectPath)\Saved\ShaderSymbols"/>
		<Property Name="GPUReshapeIniPath"        Value="$(RootDir)\Engine\Source\Programs\GPUReshape\Resources\RuntimeConfig.ini"/>

		<!-- Validation -->
		<ForEach Name="TargetPlatform" Values="$(TargetPlatforms)" Separator="+">
			<Error Message="GPU Reshape requested for platform '$(TargetPlatform)', must be one of [Win64]" If="'$(TargetPlatform)' != 'Win64'"/>
		</ForEach>

		<!-- Bootstrap arguments -->
		<Property Name="GPUReshapeArgs"           Value="-cvarsini=$(GPUReshapeIniPath) -GRS.Path=$(GPUReshapePath)\GPUReshape.exe -GRS.Workspace=$(GPUReshapeWorkspacePath) -GRS.Report=$(GPUReshapeReportPath) -GRS.SymbolPath=$(GPUReshapeSymbolPath) -nothreadtimeout -noheartbeatthread"/>
		<Property Name="GPUReshapeTestCommonArgs" Value="$(AutomatedPerformanceCommonArgs) -Bootstrap=GPUReshapeBootstrapper.exe -args=&quot;$(GPUReshapeArgs)&quot; -iterations=$(IterationsGPUReshape)"/>
	</Do>
	
	<Do If="$(UsePlatformExtraDebugMemory)">
		<Property Name="LLMTestCommonArgs"      Value="$(LLMTestCommonArgs) -AutomatedPerfTest.UsePlatformExtraDebugMemory"/>
		<Property Name="InsightsTestCommonArgs" Value="$(InsightsTestCommonArgs) -AutomatedPerfTest.UsePlatformExtraDebugMemory"/>
	</Do>

	<!-- TargetPerfTestList accumulates a list of encoded performance test description. These are decoded by the loop 
	over tests in BuildAndTestProject.

	Format: TestName{gauntlet-args}[NodeDisplayName]
	- TestName: Gauntlet test class (passed to RunUnreal -test=)
	- {gauntlet-args}: Optional command-line arguments for the test (extracted by SplitCombinedTestNameAndArgs)
	- [NodeDisplayName]: Optional friendly name for the BuildGraph node

	Example: AutomatedPerfTest.SequenceTest{-AutomatedPerfTest.MapSequenceName=MySequenceName}[Sequence Perf MySequenceName]

	BATP decodes this by:
	1. Extracting TestName (before { or [) for -test= argument
	2. Extracting arguments (within {} or []) appended to RunUnreal command
	3. Using NodeDisplayName (if present) for BuildGraph node naming
	
	This is intentionally separate from TargetTestList to enable BuildAndTestProject to filter functional tests and
	performance tests separately. This enables BATP to ensure performance tests are only run on on the intended device 
	configurations when PerfTestConfigs are in use. -->
	<Property Name="TargetPerfTestList" Value="" />

 	<Do If="$(WithAPTSequenceTests)">
		<ForEach Name="MapSequenceComboName" Values="$(MapSequenceComboNames)" Separator="+">
			<Property Name="TargetPerfTestList" Value="$(TargetPerfTestList)+AutomatedPerfTest.SequenceTest{-AutomatedPerfTest.SequencePerfTest.MapSequenceName=$(MapSequenceComboName) -AutomatedPerfTest.DoCSVProfiler $(PerfTestCommonArgs)}[Sequence Perf $(MapSequenceComboName)]" If="$(DoPerf)" />
			<Property Name="TargetPerfTestList" Value="$(TargetPerfTestList)+AutomatedPerfTest.SequenceTest{-AutomatedPerfTest.SequencePerfTest.MapSequenceName=$(MapSequenceComboName) -AutomatedPerfTest.DoCSVProfiler -AutomatedPerfTest.DoLLM $(LLMTestCommonArgs)}[Sequence LLM $(MapSequenceComboName)]" If="$(DoLLM)"/>
			<Property Name="TargetPerfTestList" Value="$(TargetPerfTestList)+AutomatedPerfTest.SequenceTest{-AutomatedPerfTest.SequencePerfTest.MapSequenceName=$(MapSequenceComboName) -AutomatedPerfTest.DoCSVProfiler -AutomatedPerfTest.DoGPUPerf $(GPUPerfTestCommonArgs)}[Sequence GPUPerf $(MapSequenceComboName)]" If="$(DoGPUPerf)" />
			<Property Name="TargetPerfTestList" Value="$(TargetPerfTestList)+AutomatedPerfTest.SequenceTest{-AutomatedPerfTest.SequencePerfTest.MapSequenceName=$(MapSequenceComboName) -AutomatedPerfTest.DoCSVProfiler -AutomatedPerfTest.DoInsightsTrace $(InsightsTestCommonArgs)}[Sequence Insights $(MapSequenceComboName)]" If="$(DoInsightsTrace)"/>
			<Property Name="TargetPerfTestList" Value="$(TargetPerfTestList)+AutomatedPerfTest.SequenceTest{-AutomatedPerfTest.SequencePerfTest.MapSequenceName=$(MapSequenceComboName) -AutomatedPerfTest.DoGPUReshape $(GPUReshapeTestCommonArgs)}[Sequence GPUReshape $(MapSequenceComboName)]" If="$(DoGPUReshape)"/>
		</ForEach>
	</Do>

	<Do If="$(WithAPTStaticCameraTests)">
		<ForEach Name="StaticCameraMapName" Values="$(StaticCameraMapNames)" Separator="+">
			<Property Name="TargetPerfTestList" Value="$(TargetPerfTestList)+AutomatedPerfTest.StaticCameraTest{-AutomatedPerfTest.StaticCameraPerfTest.MapName=$(StaticCameraMapName) -AutomatedPerfTest.DoCSVProfiler $(PerfTestCommonArgs)}[Static Camera Perf $(StaticCameraMapName)]" If="$(DoPerf)" />
			<Property Name="TargetPerfTestList" Value="$(TargetPerfTestList)+AutomatedPerfTest.StaticCameraTest{-AutomatedPerfTest.StaticCameraPerfTest.MapName=$(StaticCameraMapName) -AutomatedPerfTest.DoCSVProfiler -AutomatedPerfTest.DoLLM $(LLMTestCommonArgs)}[Static Camera LLM $(StaticCameraMapName)]" If="$(DoLLM)"/>
			<Property Name="TargetPerfTestList" Value="$(TargetPerfTestList)+AutomatedPerfTest.StaticCameraTest{-AutomatedPerfTest.StaticCameraPerfTest.MapName=$(StaticCameraMapName) -AutomatedPerfTest.DoCSVProfiler -AutomatedPerfTest.DoGPUPerf $(GPUPerfTestCommonArgs)}[Static Camera GPUPerf $(StaticCameraMapName)]" If="$(DoGPUPerf)"/>
			<Property Name="TargetPerfTestList" Value="$(TargetPerfTestList)+AutomatedPerfTest.StaticCameraTest{-AutomatedPerfTest.StaticCameraPerfTest.MapName=$(StaticCameraMapName) -AutomatedPerfTest.DoCSVProfiler -AutomatedPerfTest.DoInsightsTrace $(InsightsTestCommonArgs)}[Static Camera Insights $(StaticCameraMapName)]" If="$(DoInsightsTrace)"/>
			<Property Name="TargetPerfTestList" Value="$(TargetPerfTestList)+AutomatedPerfTest.StaticCameraTest{-AutomatedPerfTest.StaticCameraPerfTest.MapName=$(StaticCameraMapName) -AutomatedPerfTest.DoGPUReshape $(GPUReshapeTestCommonArgs)}[Static Camera GPUReshape $(StaticCameraMapName)]" If="$(DoGPUReshape)"/>
		</ForEach>
	</Do>

	<Do If="$(WithAPTMaterialTest)">
		<Property Name="TargetPerfTestList" Value="$(TargetPerfTestList)+AutomatedPerfTest.MaterialTest{-AutomatedPerfTest.DoCSVProfiler $(PerfTestCommonArgs)}[Material Perf]" If="$(DoPerf)" />
		<Property Name="TargetPerfTestList" Value="$(TargetPerfTestList)+AutomatedPerfTest.MaterialTest{-AutomatedPerfTest.DoCSVProfiler -AutomatedPerfTest.DoLLM $(LLMTestCommonArgs)}[Material LLM]" If="$(DoLLM)"/>
		<Property Name="TargetPerfTestList" Value="$(TargetPerfTestList)+AutomatedPerfTest.MaterialTest{-AutomatedPerfTest.DoCSVProfiler -AutomatedPerfTest.DoGPUPerf $(GPUPerfTestCommonArgs)}[Material GPUPerf]" If="$(DoGPUPerf)"/>
		<Property Name="TargetPerfTestList" Value="$(TargetPerfTestList)+AutomatedPerfTest.MaterialTest{-AutomatedPerfTest.DoCSVProfiler -AutomatedPerfTest.DoInsightsTrace $(InsightsTestCommonArgs)}[Material Insights]" If="$(DoInsightsTrace)"/>
		<Property Name="TargetPerfTestList" Value="$(TargetPerfTestList)+AutomatedPerfTest.MaterialTest{-AutomatedPerfTest.DoGPUReshape $(GPUReshapeTestCommonArgs)}[Material GPUReshape]" If="$(DoGPUReshape)"/>
	</Do>

	<Do If="$(WithAPTReplayTest)">
		<Property Name="TargetPerfTestList" Value="$(TargetPerfTestList)+AutomatedPerfTest.ReplayTest{-AutomatedPerfTest.ReplayPerfTest.ReplayName=&quot;$(ReplayName)&quot; -AutomatedPerfTest.DoCSVProfiler $(PerfTestCommonArgs)}[Replay Perf]" If="$(DoPerf)" />
		<Property Name="TargetPerfTestList" Value="$(TargetPerfTestList)+AutomatedPerfTest.ReplayTest{-AutomatedPerfTest.ReplayPerfTest.ReplayName=&quot;$(ReplayName)&quot; -AutomatedPerfTest.DoCSVProfiler -AutomatedPerfTest.DoLLM $(LLMTestCommonArgs)}[Replay LLM]" If="$(DoLLM)"/>
		<Property Name="TargetPerfTestList" Value="$(TargetPerfTestList)+AutomatedPerfTest.ReplayTest{-AutomatedPerfTest.ReplayPerfTest.ReplayName=&quot;$(ReplayName)&quot; -AutomatedPerfTest.DoCSVProfiler -AutomatedPerfTest.DoGPUPerf $(GPUPerfTestCommonArgs)}[Replay GPUPerf]" If="$(DoGPUPerf)"/>
		<Property Name="TargetPerfTestList" Value="$(TargetPerfTestList)+AutomatedPerfTest.ReplayTest{-AutomatedPerfTest.ReplayPerfTest.ReplayName=&quot;$(ReplayName)&quot; -AutomatedPerfTest.DoCSVProfiler -AutomatedPerfTest.DoInsightsTrace $(InsightsTestCommonArgs)}[Replay Insights]" If="$(DoInsightsTrace)"/>
		<Property Name="TargetPerfTestList" Value="$(TargetPerfTestList)+AutomatedPerfTest.ReplayTest{-AutomatedPerfTest.ReplayPerfTest.ReplayName=&quot;$(ReplayName)&quot; -AutomatedPerfTest.DoGPUReshape $(GPUReshapeTestCommonArgs)}[Replay GPUReshape]" If="$(DoGPUReshape)"/>
	</Do>

</BuildGraph>
