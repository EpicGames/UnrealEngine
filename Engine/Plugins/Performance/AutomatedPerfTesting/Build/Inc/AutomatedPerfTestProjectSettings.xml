<?xml version="1.0" encoding="utf-8"?>
<BuildGraph xmlns="http://www.epicgames.com/BuildGraph" xmlns:xsi="http://www.w3.org/2001/XMLSchema-instance" xsi:schemaLocation="http://www.epicgames.com/BuildGraph ../../../../../../../Engine/Build/Graph/Schema.xsd" >

	<!-- These are options for Automated Perf Testing that can - and often should - be overriden in a given project -->
	<Option Name="MapSequenceComboNames" DefaultValue="" Description="+ separated list of names, referencing the names of Map/Sequence combos defined in Automated Perf Test | Sequence Project Settings" />

	<Option Name="StaticCameraMapNames" DefaultValue="" Description="+ separated list of names, referencing the names of Maps defined in Automated Perf Test | Static Camera Project Settings" />

	<Option Name="ReplayName" DefaultValue="" Description="Name of replay to run when running Automated Perf Test Replay" />
</BuildGraph>
