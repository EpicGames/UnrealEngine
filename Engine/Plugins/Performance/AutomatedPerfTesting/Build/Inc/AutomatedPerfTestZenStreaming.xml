<?xml version="1.0" encoding="utf-8"?>
<BuildGraph xmlns="http://www.epicgames.com/BuildGraph" xmlns:xsi="http://www.w3.org/2001/XMLSchema-instance" xsi:schemaLocation="http://www.epicgames.com/BuildGraph ../../../../../../Engine/Build/Graph/Schema.xsd" >
    <!-- 
        This build graph is designed to run an A/B Zen Streaming tests to enable performance comparisons. 
        
        To use this build graph import it into a project's test execution BuildAndTestProject graph.
        
        Build Graph Dependencies that must be present before this script's include:                                                                                  
           * Engine/Build/Graph/Tasks/BuildAndTestProject.xml                                                          
           * Engine/Plugins/Performance/AutomatedPerfTesting/Build/Inc/AutomatedPerfTestCommonSettings.xml
        
        Zen Streaming required Parameters to set: 
           * `DoBuildZenStreaming` - true
           * `PublishCloudSnapshot` - true
           * `PublishZenSnapshot` - true
           * `RetrieveUsingCloudArtifacts` - true
           * `PublishCloudArtifacts` - true
        
        Aggregate Target(s) to include in project test execution:
            * `Zen Streaming $(ProjectName)`
            * See Lyra Sample Game example for best practices: ./Lyra/Build/LyraTests.xml 
    -->

    <!-- Zen Streaming Config -->
    <EnvVar Name="uebp_BuildRoot_Escaped"/>
    
	<Option Name="DoBuildZenStreaming" DefaultValue="false" Restrict="true|false" Description="A flag to enable ZenStreaming Building, Cooking, and Testing"/>
	<Option Name="DoLocalZenTest" DefaultValue="false" Restrict="true|false" Description="Used to flag whether or not to use a local Zen Server for testing performance."/>
	<Option Name="ZenStoreHost" DefaultValue="" Description="The hostname or ip of the target zenserver. Do not include http://" />
	<Option Name="ZenStorePort" DefaultValue="8558" Description="The port to use on the target zen server." />
	<Option Name="ZenStreamingWinTestPool" DefaultValue="" Description="Override the default Win64 Zen Streaming Test Agent Pool." />
	<Option Name="RemoteZenHost" DefaultValue="" Description="The hostname or ip of the target zenserver to import remote build snapshots" />

    <Option Name="DefaultZenStreamingTargetTestList" DefaultValue="" Description="Default Zen Streaming Target test list"/>
    <Option Name="ZenStreamingTargetTestList" DefaultValue="$(DefaultZenStreamingTargetTestList)" Description="Zen Streaming tests to run on the target platform(s). Test1+Test2 etc"/>

	<Property Name="ZenTestArgs" Value="-AutomatedPerfTest.DoBuildZenStreaming"/>
	<Property Name="ZenTestArgs" Value="$(ZenTestArgs) -AutomatedPerfTest.ZenStoreHost=&quot;$(ZenStoreHost)&quot;"/>
	<Property Name="ZenTestArgs" Value="$(ZenTestArgs) -AutomatedPerfTest.ZenStorePort=&quot;$(ZenStorePort)&quot;"/>
	<Property Name="ZenTestArgs" Value="$(ZenTestArgs) -AutomatedPerfTest.DoLocalZenTest" If="$(DoLocalZenTest)" />

	<!-- Android ADB Port Forwarding required for Host/Client communication when Zen Server is localhosted -->
	<Property Name="AndroidPortForwarderDll" Value="Engine/Source/Developer/Android/AndroidZenServerPlugin/Binaries/Win64/AndroidPortForwarder2.dll" />

    <!-- Aggregate Stubs -->
    <Property Name="GeneratedZenTargetNodes" Value="" />
	<Property Name="GeneratedZenTargetTestNodes" Value="" />

    <!-- Setup Zen Tests -->
    <!-- append plugin-provided tests to GauntletSettings' TargetTestList -->
	<Do If="$(DoBuildZenStreaming)">
		<Do If="$(WithAPTSequenceTests)">
			<ForEach Name="MapSequenceComboName" Values="$(MapSequenceComboNames)" Separator="+">
				<Property Name="ZenStreamingTargetTestList" Value="$(ZenStreamingTargetTestList)+AutomatedPerfTest.SequenceTest{-AutomatedPerfTest.SequencePerfTest.MapSequenceName=$(MapSequenceComboName) -AutomatedPerfTest.DoCSVProfiler $(PerfTestCommonArgs) $(ZenTestArgs)}[Zen Sequence Perf $(MapSequenceComboName)]" If="$(DoPerf) and $(DoBuildZenStreaming)" />
				<Property Name="ZenStreamingTargetTestList" Value="$(ZenStreamingTargetTestList)+AutomatedPerfTest.SequenceTest{-AutomatedPerfTest.SequencePerfTest.MapSequenceName=$(MapSequenceComboName) -AutomatedPerfTest.DoCSVProfiler -AutomatedPerfTest.DoLLM $(LLMTestCommonArgs) $(ZenTestArgs)}[Zen Sequence LLM $(MapSequenceComboName)]" If="$(DoLLM) and $(DoBuildZenStreaming)"/>
				<Property Name="ZenStreamingTargetTestList" Value="$(ZenStreamingTargetTestList)+AutomatedPerfTest.SequenceTest{-AutomatedPerfTest.SequencePerfTest.MapSequenceName=$(MapSequenceComboName) -AutomatedPerfTest.DoCSVProfiler -AutomatedPerfTest.DoGPUPerf $(GPUPerfTestCommonArgs) $(ZenTestArgs)}[Zen Sequence GPUPerf $(MapSequenceComboName)]" If="$(DoGPUPerf) and $(DoBuildZenStreaming)" />
				<Property Name="ZenStreamingTargetTestList" Value="$(ZenStreamingTargetTestList)+AutomatedPerfTest.SequenceTest{-AutomatedPerfTest.SequencePerfTest.MapSequenceName=$(MapSequenceComboName) -AutomatedPerfTest.DoCSVProfiler -AutomatedPerfTest.DoInsightsTrace $(InsightsTestCommonArgs) $(ZenTestArgs)}[Zen Sequence Insights $(MapSequenceComboName)]" If="$(DoInsightsTrace) and $(DoBuildZenStreaming)"/>
				<Property Name="ZenStreamingTargetTestList" Value="$(ZenStreamingTargetTestList)+AutomatedPerfTest.SequenceTest{-AutomatedPerfTest.SequencePerfTest.MapSequenceName=$(MapSequenceComboName) -AutomatedPerfTest.DoGPUReshape $(GPUReshapeTestCommonArgs) $(ZenTestArgs)}[Zen Sequence GPUReshape $(MapSequenceComboName)]" If="$(DoGPUReshape) and $(DoBuildZenStreaming)"/>
			</ForEach>
		</Do>

		<Do If="$(WithAPTStaticCameraTests)">
			<ForEach Name="StaticCameraMapName" Values="$(StaticCameraMapNames)" Separator="+">
				<Property Name="ZenStreamingTargetTestList" Value="$(ZenStreamingTargetTestList)+AutomatedPerfTest.StaticCameraTest{-AutomatedPerfTest.StaticCameraPerfTest.MapName=$(StaticCameraMapName) -AutomatedPerfTest.DoCSVProfiler $(PerfTestCommonArgs) $(ZenTestArgs)}[Zen Static Camera Perf $(StaticCameraMapName)]" If="$(DoPerf) and $(DoBuildZenStreaming)" />
				<Property Name="ZenStreamingTargetTestList" Value="$(ZenStreamingTargetTestList)+AutomatedPerfTest.StaticCameraTest{-AutomatedPerfTest.StaticCameraPerfTest.MapName=$(StaticCameraMapName) -AutomatedPerfTest.DoCSVProfiler -AutomatedPerfTest.DoLLM $(LLMTestCommonArgs) $(ZenTestArgs)}[Static Camera LLM $(StaticCameraMapName)]" If="$(DoLLM) and $(DoBuildZenStreaming)"/>
				<Property Name="ZenStreamingTargetTestList" Value="$(ZenStreamingTargetTestList)+AutomatedPerfTest.StaticCameraTest{-AutomatedPerfTest.StaticCameraPerfTest.MapName=$(StaticCameraMapName) -AutomatedPerfTest.DoCSVProfiler -AutomatedPerfTest.DoGPUPerf $(GPUPerfTestCommonArgs) $(ZenTestArgs)}[Static Camera GPUPerf $(StaticCameraMapName)]" If="$(DoGPUPerf) and $(DoBuildZenStreaming)"/>
				<Property Name="ZenStreamingTargetTestList" Value="$(ZenStreamingTargetTestList)+AutomatedPerfTest.StaticCameraTest{-AutomatedPerfTest.StaticCameraPerfTest.MapName=$(StaticCameraMapName) -AutomatedPerfTest.DoCSVProfiler -AutomatedPerfTest.DoInsightsTrace $(InsightsTestCommonArgs) $(ZenTestArgs)}[Static Camera Insights $(StaticCameraMapName)]" If="$(DoInsightsTrace) and $(DoBuildZenStreaming)"/>			
				<Property Name="ZenStreamingTargetTestList" Value="$(ZenStreamingTargetTestList)+AutomatedPerfTest.StaticCameraTest{-AutomatedPerfTest.StaticCameraPerfTest.MapName=$(StaticCameraMapName) -AutomatedPerfTest.DoGPUReshape $(GPUReshapeTestCommonArgs) $(ZenTestArgs)}[Static Camera GPUReshape $(StaticCameraMapName)]" If="$(DoGPUReshape) and $(DoBuildZenStreaming)"/>
			</ForEach>
		</Do>

		<Do If="$(WithAPTMaterialTest)">
			<Property Name="ZenStreamingTargetTestList" Value="$(ZenStreamingTargetTestList)+AutomatedPerfTest.MaterialTest{-AutomatedPerfTest.DoCSVProfiler $(PerfTestCommonArgs) $(ZenTestArgs)}[Zen Material Perf]" If="$(DoPerf) and $(DoBuildZenStreaming)"/>
			<Property Name="ZenStreamingTargetTestList" Value="$(ZenStreamingTargetTestList)+AutomatedPerfTest.MaterialTest{-AutomatedPerfTest.DoCSVProfiler -AutomatedPerfTest.DoLLM $(LLMTestCommonArgs) $(ZenTestArgs)}[Zen Material LLM]" If="$(DoLLM) and $(DoBuildZenStreaming)"/>
			<Property Name="ZenStreamingTargetTestList" Value="$(ZenStreamingTargetTestList)+AutomatedPerfTest.MaterialTest{-AutomatedPerfTest.DoCSVProfiler -AutomatedPerfTest.DoGPUPerf $(GPUPerfTestCommonArgs) $(ZenTestArgs)}[Zen Material GPUPerf]" If="$(DoGPUPerf) and $(DoBuildZenStreaming)"/>
			<Property Name="ZenStreamingTargetTestList" Value="$(ZenStreamingTargetTestList)+AutomatedPerfTest.MaterialTest{-AutomatedPerfTest.DoCSVProfiler -AutomatedPerfTest.DoInsightsTrace $(InsightsTestCommonArgs) $(ZenTestArgs)}[Zen Material Insights]" If="$(DoInsightsTrace) and $(DoBuildZenStreaming)"/>
			<Property Name="ZenStreamingTargetTestList" Value="$(ZenStreamingTargetTestList)+AutomatedPerfTest.MaterialTest{-AutomatedPerfTest.DoCSVProfiler -AutomatedPerfTest.DoGPUReshape $(GPUReshapeTestCommonArgs) $(ZenTestArgs)}[Zen Material GPUReshape]" If="$(DoGPUReshape) and $(DoBuildZenStreaming)"/>
		</Do>

		<Do If="$(WithAPTReplayTest)">
			<Property Name="ZenStreamingTargetTestList" Value="$(ZenStreamingTargetTestList)+AutomatedPerfTest.ReplayTest{-AutomatedPerfTest.ReplayPerfTest.ReplayName=&quot;$(ReplayName)&quot; -AutomatedPerfTest.DoCSVProfiler $(PerfTestCommonArgs) $(ZenTestArgs)}[Zen Replay Perf]" If="$(DoPerf) and $(DoBuildZenStreaming)"/>
			<Property Name="ZenStreamingTargetTestList" Value="$(ZenStreamingTargetTestList)+AutomatedPerfTest.ReplayTest{-AutomatedPerfTest.ReplayPerfTest.ReplayName=&quot;$(ReplayName)&quot; -AutomatedPerfTest.DoCSVProfiler -AutomatedPerfTest.DoLLM $(LLMTestCommonArgs) $(ZenTestArgs)}[Zen Replay LLM]" If="$(DoLLM) and $(DoBuildZenStreaming)"/>
			<Property Name="ZenStreamingTargetTestList" Value="$(ZenStreamingTargetTestList)+AutomatedPerfTest.ReplayTest{-AutomatedPerfTest.ReplayPerfTest.ReplayName=&quot;$(ReplayName)&quot; -AutomatedPerfTest.DoCSVProfiler -AutomatedPerfTest.DoGPUPerf $(GPUPerfTestCommonArgs) $(ZenTestArgs)}[Zen Replay GPUPerf]" If="$(DoGPUPerf) and $(DoBuildZenStreaming)"/>
			<Property Name="ZenStreamingTargetTestList" Value="$(ZenStreamingTargetTestList)+AutomatedPerfTest.ReplayTest{-AutomatedPerfTest.ReplayPerfTest.ReplayName=&quot;$(ReplayName)&quot; -AutomatedPerfTest.DoCSVProfiler -AutomatedPerfTest.DoInsightsTrace $(InsightsTestCommonArgs) $(ZenTestArgs)}[Zen Replay Insights]" If="$(DoInsightsTrace) and $(DoBuildZenStreaming)"/>
			<Property Name="ZenStreamingTargetTestList" Value="$(ZenStreamingTargetTestList)+AutomatedPerfTest.ReplayTest{-AutomatedPerfTest.ReplayPerfTest.ReplayName=&quot;$(ReplayName)&quot; -AutomatedPerfTest.DoGPUReshape $(GPUReshapeTestCommonArgs) $(ZenTestArgs)}[Zen Replay GPUReshape]" If="$(DoGPUReshape) and $(DoBuildZenStreaming)"/>
		</Do>

		<!-- Create Zen Staged Package and Execute Tests -->
		<ForEach Name="TargetPlatform" Values="$(TargetPlatforms)" Separator="+">
			<!-- Agent Setup -->
			<Property Name="CookPlatform" Value="$(TargetPlatform)" />
			<Property Name="CookPlatform" Value="Windows" If="'$(TargetPlatform)' == 'Win64'"/>
			<Property Name="CookPlatform" Value="$(CookPlatform)_$(AndroidCookFlavor)" If="'$(TargetPlatform)' == 'Android'"/>
			<Property Name="CookPlatform" Value="$(CookPlatform)Client" If="'$(TargetType)' == 'Client'" />
			<Property Name="StagedPlatformFolder" Value="$(CookPlatform)"  />
			<Property Name="IsDesktopPlatform" Value="ContainsItem('$(AllDesktopPlatforms)', '$(TargetPlatform)', '+')"/>
			<Property Name="IsMobilePlatform" Value="('$(TargetPlatform)' == 'Android' or '$(TargetPlatform)' == 'IOS')"/>
			<Property Name="IsVirtualizedPlatform" Value="ContainsItem('$(AllVirtualizedPlatforms)', '$(TargetPlatform)', '+')"/>
			<Property Name="IsLocalPlatform" Value="($(IsDesktopPlatform) Or $(IsVirtualizedPlatform))"/>
			<Property Name="IsTargetPlatformMac" Value="('$(TargetPlatform)' == 'Mac' or '$(TargetPlatform)' == 'IOS' or '$(TargetPlatform)' == 'tvOS')"/>
			<Property Name="IsTargetPlatformLinux" Value="('$(TargetPlatform)' == 'Linux')"/>
			<Property Name="IsTargetPlatformWindows" Value="(!$(IsTargetPlatformMac) and !$(IsTargetPlatformLinux))"/>

			<!-- Declare type of host required to build & cook this platform-->
			<Property Name="HostAgentType" Value="$(EditorPlatformWin64AgentOverride)" /> 
			<Property Name="HostAgentType" Value="$(EditorPlatformMacAgentOverride)" If="$(IsTargetPlatformMac)"/>
			<Property Name="HostAgentType" Value="$(EditorPlatformLinuxAgentOverride)" If="$(IsTargetPlatformLinux)"/>

			<Property Name="CompileAgentType" Value="$(HostAgentType)" />
			<Property Name="CompileAgentType" Value="Incremental$(HostAgentType)" If="$(UseIncrementalAgents)" />

			<Property Name="TestAgentType" Value="$(TargetPlatformTestAgentOverride)" />
			<Property Name="TestAgentType" Value="Test$(HostAgentType)" If="$(IsDesktopPlatform)"/>
			<Property Name="TestAgentType" Value="$(HostAgentType)" If="$(IsDesktopPlatform) and $(DontUseTestAgent)" />
			<Property Name="TestAgentType" Value="$(TestDesktopPlatformWin64AgentOverride)" If="$(IsDesktopPlatform) and '$(TestDesktopPlatformWin64AgentOverride)' != '' and $(IsTargetPlatformWindows)"/>
			<Property Name="TestAgentType" Value="$(TestDesktopPlatformLinuxAgentOverride)" If="$(IsDesktopPlatform) and '$(TestDesktopPlatformLinuxAgentOverride)' != '' and $(IsTargetPlatformLinux)"/>
			<Property Name="TestAgentType" Value="$(TestDesktopPlatformMacAgentOverride)" If="$(IsDesktopPlatform) and '$(TestDesktopPlatformMacAgentOverride)' != '' and $(IsTargetPlatformMac)"/>

			<Property Name="StageAndPackageArguments" Value="$(ExtraStageAndPackageArguments)" />
			<Property Name="StageAndPackageArguments" Value="$(StageAndPackageArguments) $(TargetPlatformFilteredExtraStageAndPackageArguments)" If="'$(TargetPlatformFilteredExtraStageAndPackageArgumentsFilter)' != '' and ContainsItem('$(TargetPlatformFilteredExtraStageAndPackageArgumentsFilter)', '$(TargetPlatform)', '+')" />

			<Property Name="SnapshotPlatformFolder" Value="$(StagedPlatformFolder)"/>
			<Property Name="SnapshotPlatformFolder" Value="$(TargetPlatform)" If="'$(TargetPlatform)' == 'Android'"/>
			
			<!-- BuildAndTestProject Dependent Node Names -->
			<Property Name="PreNodeNameWithPlatform" Value="$(PreNodeName)"/>
			<Property Name="PlatformCompileNodeName" Value="$(PreNodeNameWithPlatform)Compile $(TargetPlatform)" />
			<Property Name="PlatformCompileNodeName" Value="$(PlatformCompileNodeName);#$(PlatformCompileNodeName)" />
			<Property Name="PlatformSnapshotPublishNodeName" Value="$(PreNodeName)Snapshot Publish $(TargetPlatform)" />
			<Property Name="PlatformAgentName" Value="$(ProjectName) $(TargetPlatform)" />
			<Property Name="PlatformEditorNodeName" Value="$(PreNodeName)Compile Editor $(HostAgentType)" />
			<Property Name="PlatformToolsNodeName" Value="$(PreNodeName)Compile Tools $(HostAgentType)" />

			<Property Name="SnapshotFilenamePrefix" Value="SnapshotDescriptor"/>
			<Property Name="SnapshotFilenamePrefix" Value="$(UE_HORDE_TEMPLATEID)" If="'$(UE_HORDE_TEMPLATEID)' != ''"/>
			<Property Name="SnapshotIdentifier" Value="$(ProjectName).$(UE_HORDE_JOBID)"/>
			
			<!-- Zen Streaming Import, Stage, Publish -->
			<Agent Name="Stage and Publish $(PlatformAgentName) Zen" Type="$(CompileAgentType)">
				<Node Name="Zen Stage $(ProjectName) $(TargetPlatform)" Requires="$(PlatformCompileNodeName);$(PlatformSnapshotPublishNodeName)">
					<Property Name="ZenBuildArgs" Value="-project=$(TargetProject) -platform=$(TargetPlatform) -ScriptsForProject=$(ProjectName) $(StageAndPackageArguments)" />
					<Property Name="ZenBuildArgs" Value="$(ZenBuildArgs) -cookflavor=$(AndroidCookFlavor)" If="'$(TargetPlatform)' == 'Android'" />
					<Property Name="ZenBuildArgs" Value="$(ZenBuildArgs) -client" If="'$(TargetType)' == 'Client'" />			

					<Property Name="ZenSnapshotDescriptorFile" Value="$(NetworkSnapshotDescFileRoot)/$(uebp_BuildRoot_Escaped)/$(ProjectName)/$(SnapshotPlatformFolder)/$(Change)/$(SnapshotFilenamePrefix)-builds.json" />
					<ZenImportSnapshot Project="$(TargetProject)" SnapshotDescriptorFile="$(ZenSnapshotDescriptorFile)" DestinationProjectId="$(SnapshotIdentifier)"/>
					
					<!-- Stage -->
					<Switch>
						<Case If="$(IsMobilePlatform)">
							<ForEach Name="PackageConfiguration" Values="$(PackageConfigurations)" Separator="+">
								<Command Name="BuildCookRun" Arguments="$(ZenBuildArgs) -configuration=$(PackageConfiguration) -skipbuild -skipcook -stage -pak -package" />
								
								<!-- Copy APK and Install batch files to staged directory for complete distributable Zen streaming package -->
								<Do If="'$(TargetPlatform)' == 'Android'">
									<Copy Files="..." From="$(ProjectPath)/Binaries/Android/$(TargetName)*.apk" To="$(ProjectOutputDirectory)/$(StagedPlatformFolder)" />
                  					<Copy Files="..." From="$(ProjectPath)/Binaries/Android/Install_$(TargetName)*.bat" To="$(ProjectOutputDirectory)/$(StagedPlatformFolder)" />
									<Copy Files="..." From="$(ProjectPath)/Binaries/Android/Uninstall_$(TargetName)*.bat" To="$(ProjectOutputDirectory)/$(StagedPlatformFolder)" />
									<!-- Copy AndroidPortForwarder plugin for Zen server ADB reverse port forwarding -->
									<Copy From="$(AndroidPortForwarderDll)" To="$(ProjectOutputDirectory)/$(StagedPlatformFolder)/AndroidPortForwarder2.dll" />
								</Do>
							</ForEach>
						</Case>
						<Default>
							<Property Name="ClientStageArguments" Value="$(ZenBuildArgs) -configuration=$(PackageConfigurations) -skipbuild -skipcook -stage -pak -NoCodeSign" />
							<Command Name="BuildCookRun" Arguments="$(ClientStageArguments)" />
							<!-- Copy project file for platforms requiring staged uproject - Required for host mounting -->
							<Copy From="$(TargetProject)" To="$(ProjectOutputDirectory)/$(StagedPlatformFolder)/$(ProjectName)/$(ProjectName).uproject" If="ContainsItem('$(PlatformsRequiringStagedUproject)', '$(TargetPlatform)', '+')"/>
						</Default>
					</Switch>
				</Node>
	
				<!-- Zen Streaming Publish -->
				<Node Name="Zen Publish $(ProjectName) $(TargetPlatform) files" Requires="Zen Stage $(ProjectName) $(TargetPlatform)">
					<Property Name="TargetDirectory" Value="$(ProjectOutputDirectory)/$(StagedPlatformFolder)"/>
					<Expand Name="WriteArtifactMetadataFile" Type="staged-build" Platform="$(StagedPlatformFolder)" Configuration="$(PackageConfigurations)" ArtifactDir="$(TargetDirectory)" />
					<Tag Files="$(TargetDirectory)/..." With="#Published Zen $(ProjectName) $(TargetPlatform) Files"/>
					<Property Name="ZenArtifactMetadata" Value="cookPlatform=$(StagedPlatformFolder);dashboard-category=Staged Builds - Cloud;$(SharedCloudBuildMetadata)" />

					<CreateCloudArtifact 
						Name="$(ProjectName)-$(TargetPlatform)-Zen-Staged"
						BuildGroup="$(BuildVersion)" 
						Type="staged-build" 
						BaseDir="$(TargetDirectory)" 
						Metadata="$(ZenArtifactMetadata)"
						Project="$(TargetProject)" 
						Platform="$(StagedPlatformFolder)-Zen" 
						Files="#Published Zen $(ProjectName) $(TargetPlatform) Files"/>
					<Delete Files="$(TargetDirectory)/artifactmetadata.json"/>
				</Node>	 
			</Agent>

			<!-- Zen Streaming Import Tests -->
			<Property Name="AllPlatformTestNodes" Value="" />
			<Property Name="AllPlatformInstallNodes" Value="" />

			<ForEach Name="TestConfig" Values="$(TestConfigList)" Separator="^">
				<Property Name="TestConfigTitle" Value=""/>
				<Property Name="TestConfigArgs" Value="" />

				<Regex Input="$(TestConfig)" Pattern="(.*)\|(.*)\|(.*)\|(.*)\|(.*)\|(.*)" Capture="TestConfigName;TestConfigTargetPlatform;ConfigArtifactPostfix;ConfigAdditionalTestArguments;ConfigAdditionalAppArguments;ConfigAgentType"/>
				
				<Property Name="TestConfigTargetPlatform" Value="$(TargetPlatform)" If="'$(TestConfigTargetPlatform)' == 'All' Or '$(TestConfigTargetPlatform)' == ''"/>

				<Property Name="TestTargetAgentType" Value="$(TestAgentType)"/>
				<Property Name="TestTargetAgentType" Value="$(ConfigAgentType)" If="'$(ConfigAgentType)' != ''"/>
				<Property Name="TestTargetAgentType" Value="$(ZenStreamingWinTestPool)" If="'$(TargetPlatform)' == 'Win64' and '$(ZenStreamingWinTestPool)' != ''" /> 

				<Property Name="TestConfigArgs" Value="$(ConfigAdditionalTestArguments)" If="'$(ConfigAdditionalTestArguments)' != ''"/>
				<Property Name="TestConfigArgs" Value="$(TestConfigArgs) -ArtifactPostfix=$(ConfigArtifactPostfix)" If="'$(ConfigArtifactPostfix)' != ''"  />
				<Property Name="TestConfigArgs" Value="$(TestConfigArgs) -args=&quot;$(ConfigAdditionalAppArguments)&quot;" If="'$(ConfigAdditionalAppArguments)' != ''" />

				<Do If="'$(TestConfigTargetPlatform)' == '$(TargetPlatform)'">
					<Property Name="TestConfigTitle" Value=" $(TestConfigName)" If="'$(TestConfigName)' != ''"/>
					
					<Agent Name="Run Zen Streaming Tests $(TargetPlatform)" Type="$(TestTargetAgentType)">
						<!-- Setup Zen Args -->
						<Property Name="ZenSnapshotDescriptorFile" Value="$(NetworkSnapshotDescFileRoot)/$(uebp_BuildRoot_Escaped)/$(ProjectName)/$(SnapshotPlatformFolder)/$(Change)/$(SnapshotFilenamePrefix)-builds.json" />
						<!-- Setup Gauntlet Args -->
						<Property Name="GauntletBuildArg" Value="-build=&quot;$(ProjectOutputDirectory)/$(StagedPlatformFolder)&quot;" />
						<Property Name="GauntletTestArgs" Value="$(GauntletTargetTestArgs)"/>
						<Property Name="GauntletTestArgs" Value="$(GauntletTargetTestAltArgs)" If="ContainsItem('$(AlternateDevicePoolPlatforms)', '$(TargetPlatform)', ';')"/>
						<Property Name="GauntletTestArgs" Value="$(GauntletTestArgs) -adbkeys=&quot;$(AdbKeysPath)&quot;" If="'$(AdbKeysPath)' != '' and '$(TargetPlatform)' == 'Android'"/>
	
						<Do If="!$(Horde) And ('$(NetworkOutputDirectory)' != '' Or '$(GeneratedBuildVersion)' != '')">
							<Property Name="TestDirectory" Value="$(NetworkOutputDirectory)" />
							<Property Name="TestDirectory" Value="$(GeneratedBuildDir)" If="'$(GeneratedBuildVersion)' != ''"/>
							<Property Name="GauntletBuildArg" Value="-build=&quot;$(TestDirectory)&quot;" />
							<Property Name="GauntletBuildArg" Value="$(GauntletBuildArg) -packaged" If="$(UsePackaged)" />
							<Property Name="GauntletBuildArg" Value="$(GauntletBuildArg) -destlocalinstalldir=&quot;$(LocalInstallDirectory)&quot;" If="!$(IsLocalPlatform) And $(WithDeviceReservation)"/>
						</Do>
						
						<!-- Run through all Test Configsconfigs -->
						<ForEach Name="TestConfiguration" Values="$(TestConfigurations)" Separator="+">
							<Property Name="NodeAnnotations" Value=""/>
							<Warning Message="Test Agent Type: $(TestTargetAgentType)"/>
							
							<!-- Retrieve Staged Build-->
							<Property Name="CloudArtifactPlatformRetrieve" Value="$(PreNodeNameWithPlatform)Retrieve Build Artifact $(TargetPlatform)" />
							<Property Name="ZenTestPrerequisites" Value="Zen Publish $(ProjectName) $(TargetPlatform) files" />
							<Property Name="ZenCloudArtifactPlatformRetrieveWithConfig" Value="Zen $(CloudArtifactPlatformRetrieve) $(TestConfiguration)$(TestConfigTitle)"/>
							<Node Name="$(ZenCloudArtifactPlatformRetrieveWithConfig)" Requires="$(ZenTestPrerequisites)">
								<RetrieveCloudArtifact 
									Name="$(ProjectName)-$(TargetPlatform)-Zen-Staged" 
									Type="staged-build" 
									OutputDir="$(ProjectOutputDirectory)/$(StagedPlatformFolder)" 
									Project="$(TargetProject)" 
									Platform="$(StagedPlatformFolder)-Zen" 
									BuildGroup="$(BuildVersion)" 
									AllowMultipleMatches="true"
								/>
								<Warning Message="Test Agent Type: $(TestTargetAgentType)"/>
							</Node>
							<Property Name="ZenTestPrerequisites" Value="$(ZenTestPrerequisites);$(ZenCloudArtifactPlatformRetrieveWithConfig)" />

							<!-- Install step -->
							<Do If="!$(IsLocalPlatform) And $(WithDeviceReservation)">
								<Property Name="NodeAnnotations" Value=""/>
								<Property Name="NodeAnnotations" Value="Workflow=$(AnnotationsTarget)" If="ContainsItem('$(AnnotationsPlatforms)', '$(TargetPlatform)', ';') and ContainsItem('$(AnnotationsTests)', 'Install', ';')"/>
								<Property Name="ZenInstallNodeName" Value="Zen $(PreNodeName)$(TargetPlatform) $(TestConfiguration)$(TestConfigTitle) Install Game"/>
								<Property Name="InstallArgs" Value="-ClientCount=1 $(GauntletPreInstallArgs)"/>

								<Do If="'$(AutomationReportOutputDirectory)' != ''">
									<Property Name="OutputFragmentPath" Value="$(TargetPlatform)/$(TestConfiguration)/UE.InstallOnly" />
									<Property Name="OutputReportPath" Value="$(AutomationReportOutputDirectory)/$(OutputFragmentPath)" />
									<Property Name="GauntletReportArgs" Value="-logdir=&quot;$(OutputReportPath)&quot; -artifactname=Gauntlet" />
								</Do>

								<Node Name="$(ZenInstallNodeName)" Requires="$(ZenTestPrerequisites)" Annotations="DeviceReserve=Begin;$(NodeAnnotations)">
									<Command Name="RunUnreal" Arguments="-project=$(TargetProject) -test=&quot;UE.InstallOnly&quot; -platform=$(TargetPlatform) -configuration=$(TestConfiguration) -target=$(TargetName) $(GauntletBuildArg) $(GauntletTestArgs) $(GauntletReportArgs) $(InstallArgs)" />
								</Node>

								<Property Name="ZenTestPrerequisites" Value="$(ZenTestPrerequisites);$(ZenInstallNodeName)" />

								<Do If="!ContainsItem('$(TargetPlatformsToSkipBootTests)', '$(TargetPlatform)', '+') And !ContainsItem('$(TargetPlatformsToSkipTests)', '$(TargetPlatform)', '+')">
									<Property Name="AllPlatformInstallNodes" Value="$(AllPlatformInstallNodes);$(ZenInstallNodeName)"/>
								</Do>

								<!-- Skip install for all other test node -->
								<Property Name="GauntletTestArgs" Value="$(GauntletTestArgs) $(GauntletTestSkipInstallArgs)"/>
							</Do>

							<!-- Execute Zen Streaming Tests-->
							<ForEach Name="TestName" Values="$(ZenStreamingTargetTestList)" Separator="+">
								<Property Name="TestNameTitle" Value="$(TestName)"/>
								<Expand Name="SanitizeTestNameTitle"/>
								
								<!-- Run and report on the test-->
								<Do If="'$(AutomationReportOutputDirectory)' != ''">
									<Property Name="OutputFragmentPath" Value="$(TargetPlatform)/$(TestConfiguration)/$(TestNameTitle)" />
									<Property Name="OutputReportPath" Value="$(AutomationReportOutputDirectory)/$(OutputFragmentPath)" />
									<Property Name="GauntletReportArgs" Value="-logdir=&quot;$(OutputReportPath)&quot; -artifactname=Gauntlet -ReportExportPath=&quot;$(OutputReportPath)&quot;" />
									<Property Name="GauntletReportArgs" Value="$(GauntletReportArgs) -ReportURL=&quot;$(AutomationReportUrl)/$(OutputFragmentPath)/&quot;" If="'$(AutomationReportUrl)' != ''" />
								</Do>
								
								<Property Name="TestNodeName" Value="ZenStreaming $(PreNodeName)$(TargetPlatform) $(TestConfiguration)$(TestConfigTitle) $(TestNameTitle)" />
								<Property Name="NodeAnnotations" Value=""/>
								<Property Name="NodeAnnotations" Value="Workflow=$(AnnotationsTarget)" If="ContainsItem('$(AnnotationsPlatforms)', '$(TargetPlatform)', ';') and ContainsItem('$(AnnotationsTests)', '$(TestName)', ';')"/>
	
								<Property Name="TestNameBeforeParentheses" Value="$(TestName)"/>
								<Property Name="TestArgumentsWithinParentheses" Value=""/>
								<Expand Name="SplitCombinedTestNameAndArgs"/>
	
								<Property Name="GauntletTestArgs" Value="$(GauntletTargetTestArgs)"/>
								<Property Name="GauntletTestArgs" Value="$(GauntletTargetTestAltArgs)" If="ContainsItem('$(AlternateDevicePoolPlatforms)', '$(TargetPlatform)', ';')"/>
								<Property Name="GauntletTestArgs" Value="$(GauntletTestArgs) -adbkeys=&quot;$(AdbKeysPath)&quot;" If="'$(AdbKeysPath)' != '' and '$(TargetPlatform)' == 'Android'"/>

								<!-- 'Requires' should be the package game and the publish folder is used; however some tests may require the editor ie: UE.TargetAutomation -->
								<Node Name="$(TestNodeName)" Requires="$(PlatformEditorNodeName);$(ZenTestPrerequisites)" Annotations="$(NodeAnnotations)">
									<Property Name="ZenPlatform" Value="$(TargetPlatform)"/>
									<Property Name="ZenPlatform" Value="Windows" If="'$(TargetPlatform)' == 'Win64'"/>
									<Property Name="ZenArgs" Value="-AutomatedPerfTest.ZenStoreProject=&quot;$(SnapshotIdentifier)&quot; -AutomatedPerfTest.ZenStorePlatform=&quot;$(ZenPlatform)&quot;" />

									<!-- Copy AndroidPortForwarder plugin from staged package for Zen server ADB reverse port forwarding -->
									<Copy From="$(ProjectOutputDirectory)/$(StagedPlatformFolder)/AndroidPortForwarder2.dll" To="$(AndroidPortForwarderDll)" If="$(DoLocalZenTest) and '$(TargetPlatform)' == 'Android'" />
	
									<!-- Launch a local Zen Server, import the snapshot, execute test, cleanup snapshot -->
									<ZenLaunch Project="$(TargetProject)" If="$(DoLocalZenTest)" />
									<ZenImportSnapshot Project="$(TargetProject)" RemoteZenHost="$(ZenStoreHost)" RemoteZenPort="$(ZenStorePort)" SnapshotDescriptorFile="$(ZenSnapshotDescriptorFile)" DestinationProjectId="$(SnapshotIdentifier)" If="$(DoLocalZenTest)"/>
									<Command Name="RunUnreal" Arguments="-project=$(TargetProject) -test=&quot;$(TestNameBeforeParentheses)&quot; -platform=$(TargetPlatform) -configuration=$(TestConfiguration) -target=$(TargetName) $(GauntletBuildArg) $(GauntletTestArgs) $(TestArgumentsWithinParentheses) $(GauntletReportArgs) $(TestConfigArgs) $(ZenArgs)"/>
								</Node>

								<!-- Add it to our list -->
								<Do If="!ContainsItem('$(TargetPlatformsToSkipTests)', '$(TargetPlatform)', '+') And (!Contains('$(TestListToIncludePlatforms)', '^$(TestNameBeforeParentheses)|') Or ContainsItem('$(TestListToIncludePlatforms)', '$(TestNameBeforeParentheses)|$(TargetPlatform)', '^'))">
									<Property Name="AllPlatformTestNodes" Value="$(AllPlatformTestNodes);$(TestNodeName)"/>
								</Do>
							</ForEach>
						</ForEach>
					</Agent>
				</Do>
			</ForEach>

			<!-- Add Labels & Tasks-->
			<Label Category="Tests" Name="$(PreLabelName)Install $(TargetPlatform)" Requires="$(AllPlatformInstallNodes)" />
			<Label Category="Tests" Name="$(PreLabelName)Test $(TargetPlatform)" Requires="$(AllPlatformTestNodes)" />

			<Property Name="ZenPlatformTasks" Value="" />
			<Property Name="ZenPlatformTasks" Value="$(ZenPlatformTasks);Zen Publish $(ProjectName) $(TargetPlatform) files"/>
			<Property Name="ZenPlatformTasks" Value="$(ZenPlatformTasks);$(AllPlatformTestNodes)"/>

			<Property Name="GeneratedZenTargetNodes" Value="$(GeneratedZenTargetNodes);$(ZenPlatformTasks)"/>
			<Property Name="GeneratedZenTargetTestNodes" Value="$(GeneratedZenTargetTestNodes);$(AllPlatformTestNodes)"/>
		</ForEach>

		<Aggregate Name="Zen Streaming $(ProjectName)" Requires="$(GeneratedZenTargetNodes);$(GeneratedZenTargetTestNodes)"/>
	</Do>
</BuildGraph>
