<?xml version='1.0' ?>
<BuildGraph xmlns="http://www.epicgames.com/BuildGraph" xmlns:xsi="http://www.w3.org/2001/XMLSchema-instance" xsi:schemaLocation="http://www.epicgames.com/BuildGraph ../../../../../Build/Graph/Schema.xsd" >

	<!-- 
	PerfTestConfigList encodes a list of PerfTestConfigs - the 'device specs' for performance testing.
	
	Add elements to this list by expanding AddPerfTestConfig. Note that the encoding must match the encoding used in 
	for TestConfigs in Engine/Build/Graph/Tasks/Inc/TestConfigs.xml as the lists are combined for uniform processing in
	BuildAndTestProject. 
	-->	
	<Property Name="PerfTestConfigList" Value=""/>

	<!--
	PerfTestConfigs enable creating a named test configuration and the parameters that define it.
	
	Identity
	* TestConfigName - The unique identifier for this test configuration.
	* TestPlatform - The platform to which this configuration applies.

	Target Hardware Selection
	* AgentType - Constrain the test to run on an agent of this type as defined in Horde.
	* PerfModel - Constrain this test to run on a device with this model name in Horde.

	Device Benchmark Execution Configuration
	* TestIterationCooldownSeconds - Cooldown period between iterations if multiple iterations are requested to reduce the impact of thermal throttling.

	Runtime Configuration
	* DeviceProfile - The device profile to apply at launch.
	
	General Configuration:
	* RunUnrealArguments - Additional command line arguments passed to RunUnreal.
	* AppArguments - Additional command line arguments passed to the engine.
	
	-->
	<Macro Name="AddPerfTestConfig" Arguments="TestConfigName;TestPlatform" OptionalArguments="AgentType;RunUnrealArguments;AppArguments;PerfModel;TestIterationCooldownSeconds;DeviceProfile">
		<Property Name="_RunUnrealArguments" Value="$(RunUnrealArguments)"/>
		<Property Name="_RunUnrealArguments" Value="$(_RunUnrealArguments) -AutomatedPerfTest.TestConfigName=&quot;$(TestConfigName)&quot;"/>
		<Property Name="_RunUnrealArguments" Value="$(_RunUnrealArguments) -PerfModel=$(PerfModel)" If="'$(PerfModel)' != ''"/>
		<Property Name="_RunUnrealArguments" Value="$(_RunUnrealArguments) -TestIterationCooldownSeconds=$(TestIterationCooldownSeconds)" If="'$(TestIterationCooldownSeconds)' != ''"/>
		<Property Name="_AppArguments" Value="$(AppArguments)"/>
		<Property Name="_AppArguments" Value="$(_AppArguments) -deviceprofile=$(DeviceProfile)" If="'$(DeviceProfile)' != ''"/>
		<StringOp Input="$(TestConfigName)" Method="Replace" Arguments=" ;_" Output="_ArtifactPostfix"/>
		<Property Name="PerfTestConfigList" Value="$(PerfTestConfigList)^$(TestConfigName)|$(TestPlatform)|$(_ArtifactPostfix)|$(_RunUnrealArguments)|$(_AppArguments)|$(AgentType)"/>
	</Macro>

	<!-- 
	Enable filtering to ensure that performance tests only run on configurations added by AddPerfTestConfig.
	
	Set to true for projects using PerfTestConfigs to constrain performance tests to specific hardware configurations. 
	When enabled, performance tests from TargetPerfTestList will only execute on configurations added via 
	AddPerfTestConfig, preventing unconstrained runs that would pollute configuration-specific metric collections.

	This is explicitly set - not auto-detected - because projects may use Options to guard use of AddPerfTestConfigs.
	When filtering is enabled but all AddPerfTestConfig options are disabled (empty PerfTestConfigList), performance 
	tests will be skipped rather than running on the default config. 

	Leave false for projects that:
	- Don't use performance testing
	- Don't use PerfTestConfigs
	-->
	<Property Name="EnablePerfTestConfigFiltering" Value="false"/>

</BuildGraph>
