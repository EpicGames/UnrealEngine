// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Linq;
using Gauntlet;

namespace AutomatedPerfTest
{
	public class AutomatedMaterialPerfTestConfig : AutomatedPerfTestConfigBase
	{
	}

	/// <summary>
	/// Implementation of a Gauntlet TestNode for AutomatedPerfTest plugin
	/// </summary>
	/// <typeparam name="TConfigClass"></typeparam>
	public abstract class AutomatedMaterialPerfTestNode<TConfigClass> : AutomatedPerfTestNode<TConfigClass>, IAutomatedPerfTest
		where TConfigClass : AutomatedMaterialPerfTestConfig, new()
	{
		public AutomatedMaterialPerfTestNode(UnrealTestContext InContext) : base(InContext)
		{
			SummaryTable = "materials";
		}

		public List<string> GetTestsFromConfig()
		{
			List<string> OutMaterials = new List<string>();
			ReadConfigArray(Context,
				"/Script/AutomatedPerfTesting.AutomatedMaterialPerfTestProjectSettings",
				"MaterialsToTest",
				Material => OutMaterials.Add(Material.Replace("\"", "")));

			return OutMaterials;
		}

		public override TConfigClass GetConfiguration()
		{
			TConfigClass Config = base.GetConfiguration();

			Config.DataSourceName = Config.GetDataSourceName(Context.BuildInfo.ProjectName, "Material");

			// extend the role(s) that we initialized in the base class
			if (Config.GetRequiredRoles(UnrealTargetRole.Client).Any())
			{
				foreach(UnrealTestRole ClientRole in Config.GetRequiredRoles(UnrealTargetRole.Client))
				{
					ClientRole.Controllers.Add("AutomatedMaterialPerfTest");
				}
			}

			return Config;
		}

		protected override string GetNormalizedInsightsFileName(string CSVFileName)
		{
			return GetNormalizedInsightsFileName(CSVFileName, "_Materials");
		}
	}

	/// <summary>
	/// "Standard issue" implementation usable for samples that don't need anything more advanced
	/// </summary>
	public class MaterialTest : AutomatedMaterialPerfTestNode<AutomatedMaterialPerfTestConfig>
	{
		public MaterialTest(Gauntlet.UnrealTestContext InContext)
			: base(InContext)
		{
		}
	}
}
