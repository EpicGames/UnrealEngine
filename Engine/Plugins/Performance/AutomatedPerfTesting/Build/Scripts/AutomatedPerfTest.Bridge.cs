// Copyright Epic Games, Inc. All Rights Reserved.

using AutomatedPerfTesting;
using Gauntlet;
using System;
using System.Collections.Generic;
using System.Data;
using System.Linq;
using System.Reflection;

/// <summary>
/// This file defines the Test Bridge interface. In this context, a "Test Bridge"
/// refers to the custom test session configuration and logic which needs to run
/// before Gauntlet launches the test session on the target device. 
/// 
/// A Test Bridge may contain various attributes which enable composing and defining 
/// test-specific config and logic depending on the attribute parameters. These 
/// bridges are then instantiated for a given test if the test context parameters match
/// the attribute parameters. 
/// 
/// Simple example would be defining Test Bridges for each Test Type in your project
/// i.e. SequenceBridge, ReplayBridge, ProfileGoBridge etc. such that each bridge
/// is only activated for their corresponding test type. 
/// </summary>
namespace AutomatedPerfTest
{
	/// <summary>
	/// Interface to be used if a project wants to configure the Automated Perf Test
	/// context, roles and/or config without making intrusive changes to the main 
	/// node before launching the test. 
	/// 
	/// Ideally project/game specific command line parameters get set here. 
	/// </summary>
	public interface ITestBridge
	{
		/// <summary>
		/// Initial configure call. This is invoked first.
		/// </summary>
		/// <param name="Context">Test Context</param>
		/// <param name="Config">APT Config Base</param>
		void Configure(UnrealTestContext Context, AutomatedPerfTestConfigBase Config) { }

		/// <summary>
		/// Configure client role
		/// </summary>
		/// <param name="Role">Client Test Role</param>
		/// <param name="Context">Test Context</param>
		/// <param name="Config">APT Config Base</param>
		void ConfigureClient(UnrealTestRole Role, UnrealTestContext Context, AutomatedPerfTestConfigBase Config) { }

		/// <summary>
		/// Configure server role. This is invoked optionally if a server role is required for the test. 
		/// </summary>
		/// <param name="Role">Server Test Role</param>
		/// <param name="Context">Test Context</param>
		/// <param name="Config">APT Config Base</param>
		void ConfigureServer(UnrealTestRole Role, UnrealTestContext Context, AutomatedPerfTestConfigBase Config) { }

		/// <summary>
		/// If this is true, server role is instantiated and configured. 
		/// </summary>
		/// <returns></returns>
		bool IsServerRoleRequired() { return false; }
	}

	/// <summary>
	/// Null Test Bridge
	/// </summary>
	public class NullBridge : ITestBridge { }


	/// <summary>
	/// Static helper class to activate and invoke test bridge configuration
	/// functions.
	/// </summary>
	public static class AutoTestBridge
	{
		private static HashSet<ITestBridge> Bridges = new HashSet<ITestBridge>();

		/// <summary>
		/// Initialize Test Bridge instances with given list of Bridge class names. 
		/// </summary>
		/// <param name="InConfigBridges">Bridge names</param>
		public static void Initialize(IEnumerable<string> InConfigBridges)
		{
			foreach (string BridgeName in InConfigBridges)
			{
				ITestBridge ConfigBridge = GetConfigBridge(BridgeName.Trim());
				AddBridge(ConfigBridge);
			}
		}

		public static void AddBridge(ITestBridge ConfigBridge)
		{
			if(ConfigBridge != null)
			{
				Bridges.Add(ConfigBridge);
				Log.Info($"Test Bridge Added: {ConfigBridge.GetType().Name}");
			}
		}

		private static ITestBridge GetConfigBridge(string ConfigBridgeName)
		{
			if (!string.IsNullOrEmpty(ConfigBridgeName))
			{
				Type BridgeType = Util.GetTypeWithInterface<ITestBridge>(ConfigBridgeName);
				if (BridgeType != null)
				{
					return Activator.CreateInstance(BridgeType) as ITestBridge;
				}
			}

			return null;
		}

		public static void Configure(UnrealTestContext Context, AutomatedPerfTestConfigBase Config)
		{
			Bridges.ToList().ForEach(Bridge => Bridge?.Configure(Context, Config));
		}

		public static void ConfigureClient(UnrealTestRole Role, UnrealTestContext Context, AutomatedPerfTestConfigBase Config)
		{
			Bridges.ToList().ForEach(Bridge => Bridge?.ConfigureClient(Role, Context, Config));
		}

		public static void ConfigureServer(UnrealTestRole Role, UnrealTestContext Context, AutomatedPerfTestConfigBase Config)
		{
			Bridges.ToList().ForEach(Bridge => Bridge?.ConfigureServer(Role, Context, Config));
		}

		public static bool IsServerRoleRequired()
		{
			return Bridges.Where(B => B.IsServerRoleRequired()).Any();
		}

		private static bool ValidateAttributes(Type TestBridge, string ProjectName = "")
		{
			EnableForProjectAttribute ProjectAttribute = TestBridge.GetCustomAttribute<EnableForProjectAttribute>();
			EnableForParamsAttribute Params = TestBridge.GetCustomAttribute<EnableForParamsAttribute>();

			AutoBridge BridgeAttribute = TestBridge.GetCustomAttribute<AutoBridge>();
			IEnumerable<Attribute> AutoBridgeTestAttribute = TestBridge.GetCustomAttributes(typeof(AutoBridgeForTestAttribute<>));

			bool Result = true;
			if (ProjectAttribute != null)
			{
				Result = ProjectAttribute.ProjectName == ProjectName;
			}

			if (Params != null)
			{
				foreach (KeyValuePair<string, string> Args in Params.Params)
				{
					string Value = Globals.Params.ParseValue(Args.Key, "");
					bool bContainsKey = Globals.Params.ParseParam(Args.Key);

					// Some params may be bool params and may only have the param key to 
					// indicate if it should be activated. 
					bool bIsBoolVal = string.IsNullOrEmpty(Value) && string.IsNullOrEmpty(Args.Value) && bContainsKey;
					bool bIsEqual = string.Equals(Value, Args.Value, StringComparison.OrdinalIgnoreCase);
					if (!bIsBoolVal && !bIsEqual)
					{
						Result = false;
						break;
					}
				}
			}

			return Result && (BridgeAttribute != null || AutoBridgeTestAttribute.Any());
		}

		/// <summary>
		/// Registers valid Test Bridge types which will then be used to configure the test 
		/// depending on the need of the test and/or project. 
		/// Following attributes used for validation: <br/>
		/// 
		/// [AutoBridge] - This enables the bridge for all automated perf tests.<br/>
		/// [AutoBridgeForTest[<paramref name="CurrentTestType"/>]] - This enables the bridge only 
		/// for the specified test type. Multiple types are supported. <br/>
		/// [EnableForProject] - Ensures the bridge is only enabled for a given project. <br/>
		/// [EnableForParams]
		/// 
		/// </summary>
		/// <param name="CurrentTestType">Type of test being run in this given session</param>
		/// <param name="ProjectName">Name of project. This should match the name in 
		/// EnableForProjectAttribute to take effect.</param>
		public static void ActivateTestBridges(Type CurrentTestType, string ProjectName = "")
		{
			Type[] TestBridgeTypes = Util.GetTypesWithInterface<ITestBridge>();
			foreach(Type TestBridge in TestBridgeTypes)
			{
				AutoBridge BridgeAttribute = TestBridge.GetCustomAttribute<AutoBridge>();
				IEnumerable<Attribute> AutoBridgeTestAttribute = TestBridge.GetCustomAttributes(typeof(AutoBridgeForTestAttribute<>));

				// A bridge is valid if each present attribute is valid. If an attribute is null, we do not consider
				// it for validation. 
				if (!ValidateAttributes(TestBridge, ProjectName))
				{
					continue;
				}

				if (BridgeAttribute != null)
				{
					// If a Test Bridge is enabled for all test types, we simply add the bridge
					// and continue to the next bridge derived type. 
					AddBridge(Activator.CreateInstance(TestBridge) as ITestBridge);
					continue;
				}

				// Register bridges which have indicated that they are applicable for a given type of
				// test. 
				foreach (Attribute AutoBridge in AutoBridgeTestAttribute)
				{
					if (AutoBridge != null && AutoBridge.GetType().IsGenericType) 
					{
						Type CandidateTestType = AutoBridge.GetType().GetGenericArguments().First();
						if(CandidateTestType != null && CurrentTestType == CandidateTestType)
						{
							AddBridge(Activator.CreateInstance(TestBridge) as ITestBridge);
						}
					}
				}
			}
		}
	}

	/// <summary>
	/// Automatically activate given Test Bridge for all Automated Perf Tests.
	/// If this is the only attribute present, it will be added to all
	/// tests by default. <br/>
	/// Example: <code>[AutoBridge]</code>
	/// </summary>
	[AttributeUsage(AttributeTargets.Class, AllowMultiple = false)]
	public class AutoBridge : Attribute
	{
		public AutoBridge() { }
	}

	/// <summary>
	/// Automatically activate given Test Bridge for given Automated Perf Test
	/// type. If this is the only attribute present, it will be added to
	/// all instances of a given test type.<br/>
	/// Example: <code>[AutoBridgeForTest[ReplayTest]]</code>
	/// </summary>
	[AttributeUsage(AttributeTargets.Class, AllowMultiple = true)]
	public class AutoBridgeForTestAttribute<T> : Attribute
		where T : IAutomatedPerfTest
	{
		public Type TestType { get; private set; }

		public AutoBridgeForTestAttribute()
		{ 
			TestType = typeof(T);
		}
	}

	/// <summary>
	/// Ensures test bridge is activated only for a given project. This ensures if
	/// a given bridge is very specific and/or does not apply to any other project
	/// it can only be activated for that project. <br/> 
	/// This should be combined with one of the AutoBridge attribute types to ensure
	/// auto-activation of the bridge. <br/>
	/// Example: <code>[EnableForProject("MyProjectName")]</code>
	/// </summary>
	[AttributeUsage(AttributeTargets.Class, AllowMultiple = true)]
	public class EnableForProjectAttribute : Attribute
	{
		public string ProjectName { get; private set; }

		public EnableForProjectAttribute(string InProjectName)
		{
			ProjectName = InProjectName;
		}
	}

	/// <summary>
	/// If a test bridge only applies for a very specific set of parameters,
	/// they can be specified here. <br/>
	/// Example: <code>[EnableForParams("Mode=Test", "SomeBoolVal", ...)]</code>
	/// </summary>
	[AttributeUsage(AttributeTargets.Class, AllowMultiple = true)]
	public class EnableForParamsAttribute : Attribute
	{
		public List<KeyValuePair<string, string>> Params { get; private set; }

		public EnableForParamsAttribute(params string[] InParams)
		{
			Params = InParams
				.Select(P => P.Split('='))
				.Select(S => new KeyValuePair<string, string>(S[0], S.Length > 1 ? S[1] : null))
				.ToList();
		}
	}
}