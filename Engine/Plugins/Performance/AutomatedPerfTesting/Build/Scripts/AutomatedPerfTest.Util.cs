// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Reflection;

namespace AutomatedPerfTesting
{
	/// <summary>
	/// Internal Utility Class
	/// </summary>
	internal static class Util
	{
		private static Assembly[] AssemblyCache = null;

		private static Assembly[] GetAssemblies()
		{
			if (AssemblyCache == null)
			{
				AssemblyCache = AppDomain.CurrentDomain.GetAssemblies().Reverse().ToArray();
			}

			return AssemblyCache;
		}

		/// <summary>
		/// Get array of types which match the given predicate.
		/// </summary>
		/// <param name="Predicate">Conditional predicate</param>
		/// <returns>Array of types</returns>
		public static Type[] GetTypes(Func<Type, bool> Predicate)
		{
			// Even though we cache the assemblies here, this function
			// is still iterating through all types available within the
			// assemblies to find matches. 
			List<Type> FoundTypes = new List<Type>();
			foreach (Assembly CandidateAssembly in GetAssemblies())
			{
				Type[] CandidateTypes = CandidateAssembly.GetTypes();
				List<Type> Types = CandidateTypes
						.Where(Predicate)
						.ToList();

				FoundTypes.AddRange(Types);
			}

			return FoundTypes.ToArray();
		}

		/// <summary>
		/// Get reflected Type info of given type name if available.
		/// </summary>
		/// <typeparam name="TInterface">Interface the type is derived from</typeparam>
		/// <param name="Name">Name of type</param>
		/// <returns>Returns instance of Type if available, otherwise returns null</returns>
		public static Type GetTypeWithInterface<TInterface>(string Name)
		{
			return GetTypeByName(Name, typeof(TInterface));
		}

		/// <summary>
		/// Get reflected Type info of given type name if available. Searches assemblies
		/// in current app domain. 
		/// </summary>
		/// <param name="Name">Name of type</param>
		/// <param name="InterfaceType">Interface the type is derived from. Optional</param>
		/// <returns>Returns instance of Type if available, otherwise returns null</returns>
		public static Type GetTypeByName(string Name, Type InterfaceType = null)
		{
			if (InterfaceType == null)
			{
				InterfaceType = typeof(object);
			}

			foreach (Assembly CandidateAssembly in GetAssemblies())
			{
				Type CandidateType = CandidateAssembly.GetType(Name.Trim());
				if (CandidateType != null && InterfaceType.IsAssignableFrom(CandidateType))
				{
					return CandidateType;
				}
			}

			return null;
		}

		/// <summary>
		/// Get array of types which is assignable from given interface
		/// type. 
		/// </summary>
		/// <typeparam name="TInterface">Interface the types should implement</typeparam>
		/// <returns></returns>
		public static Type[] GetTypesWithInterface<TInterface>()
		{
			Type InterfaceType = typeof(TInterface);
			return GetTypes(Type => Type.IsClass && InterfaceType.IsAssignableFrom(Type));
		}

		public static Type[] GetTypesWithAttribute<TAttribute>()
		{
			return GetTypesWithAttribute(typeof(TAttribute));
		}

		public static Type[] GetTypesWithAttribute(Type AttributeType)
		{
			return GetTypes(Type => Type.IsClass && Attribute.IsDefined(Type, AttributeType));
		}
	}
}
