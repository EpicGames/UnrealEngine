// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using EpicGame;
using UnrealBuildBase;
using Gauntlet;
using Microsoft.Extensions.Logging;
using Log = EpicGames.Core.Log;

namespace AutomatedPerfTest
{
	public class AutomatedPerfTestConfigBase : EpicGameTestConfig
	{
		/// <summary>
		/// Used to override the project name portion of the data source name
		/// Will be inserted into Automation.{dataSourceProjectName}.{dataSourceType} to construct the full data source name for the test. 
		/// This is useful when the actual project differs from the name displayed downstream in dashboards.
		/// 
		/// Values must follow rules similar to standard ProjectNames: 
		///  1. Must start with a letter.
		///  2. Can only use alpha numeric characters and _.
		///  3. Must be 20 characters or less.
		///  
		/// The full string can be overridden with AutomatedPerfTest.DataSourceNameOverride 
		/// </summary>
		[AutoParamWithNames("AutomatedPerfTest.DataSourceProjectNameOverride")]
		public string DataSourceProjectNameOverride = "";
		
		/// <summary>
		/// Used to override the test controller portion of the data source name
		/// Will be appended to Automation.ProjectName to construct the full data source name for the test
		/// 
		/// The full string can be overridden with AutomatedPerfTest.DataSourceNameOverride 
		/// </summary>
		[AutoParamWithNames("AutomatedPerfTest.DataSourceTypeOverride")]
		public string DataSourceTypeOverride = "";
		
		/// <summary>
		/// Fully override the data source name
		/// </summary>
		[AutoParamWithNames("AutomatedPerfTest.DataSourceNameOverride")]
		public string DataSourceNameOverride = "";

		/// <summary>
		/// Name of the test, useful for identifying it later
		/// </summary>
		[AutoParamWithNames("AutomatedPerfTest.TestID")]
		public string TestID = "";
		
		/// <summary>
		/// If data from the test should be gathered with Unreal Insights
		/// </summary>
		[AutoParamWithNames(false, "AutomatedPerfTest.DoInsightsTrace")]
		public bool DoInsightsTrace;

		/// <summary>
		/// Which trace channels to test with
		/// </summary>
		[AutoParamWithNames("AutomatedPerfTest.TraceChannels")]
		public string TraceChannels = "default,screenshot,stats";
		
		/// <summary>
		/// If data from the test should be gathered with the CSV Profiler
		/// </summary>
		[AutoParamWithNames(false, "AutomatedPerfTest.DoCSVProfiler")]
		public bool DoCSVProfiler;
		
		/// <summary>
		/// If data from the test should be gathered with FPS Charts
		/// </summary>
		[AutoParamWithNames(false, "AutomatedPerfTest.DoFPSChart")]
		public bool DoFPSChart;

		/// <summary>
		/// Whether to wait for a debugger to attach to an APT launched test
		/// </summary>
		[AutoParamWithNames(false, "AutomatedPerfTest.WaitForAttach")]
		public bool WaitForAttach;

		/// <summary>
		/// List of '+' separated platforms on which to use the null online subsystem service
		/// </summary>
		[AutoParamWithNames("AutomatedPerfTest.NullOSSPlatforms")]
		public string NullOSSPlatforms;

		/// <summary>
		/// Whether to run the test with -llm
		/// </summary>
		[AutoParamWithNames(false, "AutomatedPerfTest.DoLLM")]
		public bool DoLLM;

		/// <summary>
		/// Whether to run GPU perf test, that is, dynres is fixed, while GPU async work gets reduced, serializing passes to be timed separately
		/// </summary>
		[AutoParamWithNames(false, "AutomatedPerfTest.DoGPUPerf")]
		public bool DoGPUPerf;

		/// <summary>
		/// Whether to run GPU Reshape's instrumentation on the target
		/// </summary>
		[AutoParamWithNames(false, "AutomatedPerfTest.DoGPUReshape")]
		public bool DoGPUReshape;

		/// <summary>
		/// The assigned GPU Reshape workspace
		/// </summary>
		[AutoParamWithNames("BasicWorkspace", "AutomatedPerfTest.GPUReshapeWorkspace")]
		public string GPUReshapeWorkspace;

		/// <summary>
		/// Whether to run Zen Streaming Import tests with -DoBuildZenStreaming
		/// </summary>
		[AutoParamWithNames(false, "AutomatedPerfTest.DoBuildZenStreaming")]
		public bool DoBuildZenStreaming;

		/// <summary>
		/// If enabled, runs with -deterministic 
		/// </summary>
		[AutoParamWithNames(true, "AutomatedPerfTest.Deterministic")]
		public bool Deterministic;

		/// <summary>
		/// The IP/Web Address of the Zen Server
		/// </summary>
		[AutoParamWithNames("", "AutomatedPerfTest.ZenStoreHost")]
		public string ZenStoreHost;

		/// <summary>
		/// The port of the Zen Server
		/// </summary>
		[AutoParamWithNames("", "AutomatedPerfTest.ZenStorePort")]
		public string ZenStorePort;

		/// <summary>
		/// The project identifier for the Zen Server
		/// </summary>
		[AutoParamWithNames("", "AutomatedPerfTest.ZenStoreProject")]
		public string ZenStoreProject;

		/// <summary>
		/// The platform identifier for the Zen Server
		/// </summary>
		[AutoParamWithNames("", "AutomatedPerfTest.ZenStorePlatform")]
		public string ZenStorePlatform;

		/// <summary>
		/// Whether or not to run a test against a local Zen Server
		/// </summary>
		[AutoParamWithNames(false, "AutomatedPerfTest.DoLocalZenTest")]
		public bool DoLocalZenTest;

		/// <summary>
		/// Whether to lock dynamic resolution. This is applied automatically when GPUPerf test is run.
		/// </summary>
		[AutoParamWithNames(false,"AutomatedPerfTest.LockDynamicRes")]
		public bool LockDynamicRes;

		/// <summary>
		/// Path where Perf Test artifacts (Logs, CSVs, Reports, Cache) will be stored. 
		/// </summary>
		[AutoParamWithNames("", "AutomatedPerfTest.PerfOutputPath")]
		public string PerfOutputPath;

		/// <summary>
		/// Let BuildGraph tell us where we should output the Insights trace after running a test so that we know where to grab it from when we're done
		/// </summary>
		[AutoParamWithNames("", "AutomatedPerfTest.PerfCacheRoot")]
		public string PerfCacheRoot;

		/// <summary>
		/// Path to a JSON file with ignored issues (ensures, warnings, errros). Can be used to suppress hard-to-fix issues, on a per-branch basis
		/// </summary>
		[AutoParamWithNames("", "AutomatedPerfTest.IgnoredIssuesConfigAbsPath")]
		public string IgnoredIssuesConfigAbsPath;
		
		/// <summary>
		/// If we should trigger a video capture
		/// </summary>
		[AutoParamWithNames(false, "AutomatedPerfTest.DoVideoCapture")]
		public bool DoVideoCapture;
		
		/// <summary>
		/// If true, will look for the shipping configuration of Unreal Insights in order to parse Insights trace files
		/// This should be True unless you need to test an issue in parsing the Insights file itself.
		/// </summary>
		[AutoParamWithNames(false, "AutomatedPerfTest.UseShippingInsights")]
		public bool UseShippingInsights;

		/// <summary>
		/// If true, this will pass `-WaitForDebugger` to Client command line parameters. This is helpful for local 
		/// testing and local dev iteration. 
		/// </summary>
		[AutoParamWithNames(false, "AutomatedPerfTest.WaitForDebugger")]
		public bool WaitForDebugger;

		/// <summary>
		/// Used to find the report type and report graph xmls for generating local reports
		/// Defaults to those provided in the AutomatedPerfTesting plugin
		/// </summary>
		[AutoParamWithNames("",  "AutomatedPerfTest.ReportConfigDir")]
		public string ReportConfigDir;
		
		/// <summary>
		/// Used to specify where performance reports should be saved
		/// </summary>
		[AutoParamWithNames("",  "AutomatedPerfTest.ReportPath")]
		public string ReportPath;

		/// <summary>
		/// Specify whether this test requires additional debug memory
		/// on target device. The allocation of this memory is entirely
		/// dependent on the target device and may not be guaranteed.
		/// </summary>
		[AutoParamWithNames(false, "AutomatedPerfTest.UsePlatformExtraDebugMemory")]
		public bool UsePlatformExtraDebugMemory;
		
		/// <summary>
		/// If true, logs will not be considered when passing or failing
		/// a test in Test builds. Useful if logging is disabled in Test
		/// builds and we do not want to fail the test if no logs are 
		/// found. 
		/// </summary>
		[AutoParamWithNames(true, "AutomatedPerfTest.IgnoreTestBuildLogging")]
		public bool IgnoreTestBuildLogging;

		/// <summary>
		/// Used to specify a different test name in the CSV meta data, primarily used to distinguish between RHIs
		/// </summary>
		[AutoParamWithNames("", "AutomatedPerfTest.TestConfigName")]
		public string TestConfigName = "";

		/// <summary>
		/// Overrides test build version. If empty, default build version will be used. 
		/// </summary>
		[AutoParamWithNames("", "AutomatedPerfTest.TestBuildVersion")]
		public string TestBuildVersion = "";

		/// <summary>
		/// Overrides build version. If empty, default build version will be used. 
		/// </summary>
		[AutoParamWithNames("", "AutomatedPerfTest.BuildVersion")]
		public string BuildVersion = "";

		/// <summary>
		/// Comma separated list of Bridges which will be initialized and invoked during configuration.
		/// </summary>
		[AutoParamWithNames("AutomatedPerfTest.NullBridge", "AutomatedPerfTest.ConfigBridges")]
		public string ConfigBridges = "";

		/// <summary>
		/// Overrides managed account pool instance with the one provided. 
		/// </summary>
		[AutoParamWithNames("", "AutomatedPerfTest.ManagedAccountPool")]
		public string ManagedAcountPool;

		/// <summary>
		/// Optional CVars to be passed to client, comma separated. 
		/// </summary>
		[AutoParamWithNames("", "AutomatedPerfTest.CVars")]
		public string CVars;

		/// <summary>
		/// Overrides default artifact output path by passing the artifact path to the client instance. 
		/// This is useful if the project has set the artifact to output to a non-standard default output 
		/// path causing report generation to fail because of missing CSV artifacts.
		/// </summary>
		[AutoParamWithNames(false, "AutomatedPerfTest.OverrideArtifactPath")]
		public bool OverrideArtifactOutputPath;

		public string DataSourceName;

		/// <summary>
		/// Call this in the test node's GetConfiguration function to set the DataSourceName used for later data processing
		/// </summary>
		/// <param name="ProjectName">Name of the project</param>
		/// <param name="DataSourceType">Which test controller is being used to generate the data</param>
		/// <returns> the properly formatted data source name </returns>
		public string GetDataSourceName(string ProjectName, string DataSourceType="")
		{
			// if the name has been fully overridden by the calling process, just use that
			if (!string.IsNullOrEmpty(DataSourceNameOverride))
			{
				return DataSourceNameOverride;
			}
			
			// otherwise, if the data source type has been override, use either that or the one passed into this function

			if (string.IsNullOrEmpty(DataSourceType) && string.IsNullOrEmpty(DataSourceTypeOverride))
			{
				Log.Logger.LogError("No DataSourceType or DataSourceTypeOverride has been provided.");
				return null;
			}

			// If DataSourceProjectNameOverride is set, ensure it is sufficiently similar to a ProjectName.
			if (!string.IsNullOrWhiteSpace(DataSourceProjectNameOverride))
			{
				string reason;
				if (!IsProjectNameLike(DataSourceProjectNameOverride, out reason))
				{
					Log.Logger.LogError($"Invalid DataSourceProjectNameOverride: {reason}");
					return null;
				}
			}
			
			string dataSourceProjectName = string.IsNullOrWhiteSpace(DataSourceProjectNameOverride) ? ProjectName : DataSourceProjectNameOverride;
			string dataSourceType = string.IsNullOrEmpty(DataSourceTypeOverride) ? DataSourceType : DataSourceTypeOverride;
			
			return $"Automation.{dataSourceProjectName}.{dataSourceType}";
		}

		public override void ApplyToConfig(UnrealAppConfig AppConfig, UnrealSessionRole ConfigRole, IEnumerable<UnrealSessionRole> OtherRoles)
		{
			// disable MCP login
			NoMCP = true;
			RequiresLogin = false;

			base.ApplyToConfig(AppConfig, ConfigRole, OtherRoles);

			// Let the app know if we need to use extra debug memory before we 
			// instantiate it. 
			AppConfig.UsePlatformExtraDebugMemory = UsePlatformExtraDebugMemory;
		}


		/// <summary>
		///  Validation function to ensure a project name override is valid. This mirrors relevant validation in 
		///  GameProjectUtils::IsValidProjectFileForCreation valid overrides should follow the same set of constraints.
		/// </summary>
		/// <returns></returns>
		private bool IsProjectNameLike(string ProjectName, out string OutFailReason)
		{
			// Mirroring MAX_PROJECT_NAME_LENGTH
			const int maxProjectNameLength = 20;

			if ( string.IsNullOrEmpty(ProjectName) )
			{
				OutFailReason = "You must specify a project name.";
				return false;
			}

			if ( ProjectName.Length > maxProjectNameLength )
			{
				OutFailReason = $"Project names must not be longer than {maxProjectNameLength} characters.";
				return false;
			}

			if ( !char.IsLetter(ProjectName[0]) )
			{
				OutFailReason = "Project names must begin with an alphabetic character.";
				return false;
			}

			// Constrain allowed characters to alpha numeric and '_' to match NameContainsOnlyLegalCharacters.
			HashSet<char> illegalNameCharacters = new HashSet<char>();
			foreach (char c in ProjectName)
			{
				if ( char.IsLetterOrDigit(c) || c == '_')
				{
					continue;
				}

				illegalNameCharacters.Add(c);
			}
			if ( illegalNameCharacters.Count > 0)
			{
				string uniqueCharacters = string.Join("", illegalNameCharacters.OrderBy( c => c));
				OutFailReason = $"Project names may not contain the following characters: '{uniqueCharacters}'";
				return false;
			}

			OutFailReason = "";
			return true;
		}
	}
}
