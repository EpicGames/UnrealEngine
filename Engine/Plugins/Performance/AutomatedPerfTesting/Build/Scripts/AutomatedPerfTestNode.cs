// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using AutomationTool;
using Gauntlet;
using EpicGames.Core;
using Log = Gauntlet.Log;
using UnrealBuildBase;	// for Unreal.RootDirectory
using UnrealBuildTool;	// for UnrealTargetPlatform

using static AutomationTool.CommandUtils;
using System.Diagnostics;
using System.Reflection;
using System.Text;
using static Gauntlet.HordeReport.AutomatedTestSessionData;
using AutomatedTestSessionData = Gauntlet.HordeReport.AutomatedTestSessionData;
using AutomatedPerfTesting;

namespace AutomatedPerfTest
{
	public interface IAutomatedPerfTest
	{
		List<string> GetTestsFromConfig();
	}

	/// <summary>
	/// Implementation of a Gauntlet TestNode for AutomatedPerfTest plugin
	/// </summary>
	/// <typeparam name="TConfigClass"></typeparam>
	public abstract class AutomatedPerfTestNode<TConfigClass> : UnrealTestNode<TConfigClass>
		where TConfigClass : AutomatedPerfTestConfigBase, new()
	{
		public string SummaryTable = "default";
		public AutomatedPerfTestNode(UnrealTestContext InContext) : base(InContext)
		{
			// We need to save off the build name as if this is a preflight that suffix will be stripped
			// after GetConfiguration is called. This will cause a mismatch in CreateReport.
			OriginalBuildName = Globals.Params.ParseValue("BuildName", InContext.BuildInfo.BuildName);
			Log.Info("Setting OriginalBuildName to {OriginalBuildName}", OriginalBuildName);
			
			TestGuid = Guid.NewGuid();
			Log.Info("Your Test GUID is :\n" + TestGuid.ToString() + '\n');

			InitHandledErrors();
			AutoTestBridge.ActivateTestBridges(GetType(), InContext.BuildInfo.ProjectName);

			LogParser = null;
		}

		public override bool StartTest(int Pass, int InNumPasses)
		{
			LogParser = null;
			return base.StartTest(Pass, InNumPasses);
		}

		public class HandledError
		{
			public string ClientErrorString;
			public string GauntletErrorString;

			/// <summary>
			/// String name for the log category that should be used to filter errors. Defaults to null, i.e. no filter.
			/// </summary>
			public string CategoryName;

			// If error is verbose, will output debugging information such as state
			public bool Verbose;

			public HandledError(string ClientError, string GauntletError, string Category, bool VerboseIn = false)
			{
				ClientErrorString = ClientError;
				GauntletErrorString = GauntletError;
				CategoryName = Category;
				Verbose = VerboseIn;
			}
		}

		/// <summary>
		/// Should we archive dev artifacts even if -dev is passed down.
		/// </summary>
		private readonly bool ForceArchiveDevArtifacts = true;

		/// <summary>
		/// List of errors with special-cased gauntlet messages.
		/// </summary>
		public List<HandledError> HandledErrors { get; set; }

		/// <summary>
		/// Guid associated with each test run for ease of differentiation between different runs on same build.
		/// </summary>
		public Guid TestGuid { get; protected set; }

		/// <summary>
		/// Base artifact output path for current instance of this test node.
		/// </summary>
		public string BaseOutputPath { get; protected set; }

		/// <summary>
		/// The Horde job link.
		/// </summary>
		public string HordeJobLink { get; protected set; }

		/// <summary>
		/// Returns the APT Performance Artifact Path. 
		/// </summary>
		/// <param name="Platform"></param>
		/// <returns>{ProjectName}/Saved/Performance/{SubTest}/{Platform}</returns>
		public string GetPerformanceReportArtifactOutputPath(UnrealTargetPlatform Platform, TConfigClass Config = null)
		{
			if (BaseOutputPath == null)
			{
				return TempPerfCSVDir.FullName;
			}

			return Path.Combine(BaseOutputPath, GetSubtestName(Config ?? GetCachedConfiguration()), Platform.ToString());
		}

		/// <summary>
		/// Track client log messages that have been written to the test logs.
		/// </summary>
		private UnrealLogStreamParser LogParser;

		/// <summary>
		// Temporary directory for perf report CSVs
		/// </summary>
		private DirectoryInfo TempPerfCSVDir => new DirectoryInfo(Path.Combine(Unreal.RootDirectory.FullName, "GauntletTemp", "PerfReportCSVs"));

		/// <summary>
		// Holds the build name as is, since if this is a preflight the suffix will be stripped after GetConfiguration is called.
		/// </summary>
		private string OriginalBuildName = null;

		/// <summary>
		/// If true, local reports will be generated at the end of the perf test.
		/// </summary>
		private bool GenerateLocalReport = false;

		/// <summary>
		/// If true, the resulting CSV files will be imported via Perf Report Server
		/// importer. 
		/// </summary>
		private bool GeneratePRSReport = false;

		/// <summary>
		/// If true, the resulting CSV files will be emitted to Horde Telemetry.
		/// </summary>
		private bool GenerateHordeTelemetry = false;

		/// <summary>
		/// To ensure that PRS report is added only once per report
		/// </summary>
		private bool IsPRSAdded = false;

		/// <summary>
		/// Set up the base list of possible expected errors, plus the messages to deliver if encountered.
		/// </summary>
		protected virtual void InitHandledErrors()
		{
			HandledErrors = new List<HandledError>();
		}

		protected virtual string GetNormalizedInsightsFileName(string FileName) => FileName.Replace(".csv", ".utrace");

		protected string GetNormalizedInsightsFileName(string CSVFileName, string TestTypeLiteral)
		{
			int Index = CSVFileName.IndexOf(TestTypeLiteral);

			if (Index < 0)
			{
				Index = CSVFileName.ToLower().IndexOf(".csv");
			}
			else
			{
				Index = Index + TestTypeLiteral.Length;
			}

			return CSVFileName.Substring(0, Index) + ".utrace";
		}

		/// <summary>
		/// Periodically called while test is running. Updates logs.
		/// </summary>
		public override void TickTest()
		{
			IAppInstance App = null;

			if (TestInstance.ClientApps == null)
			{
				App = TestInstance.ServerApp;
			}
			else
			{
				if (TestInstance.ClientApps.Length > 0)
				{
					App = TestInstance.ClientApps.First();
				}
			}

			if (App != null)
			{
				if (LogParser == null)
				{
					LogParser = new UnrealLogStreamParser(App.GetLogBufferReader());
				}
				LogParser.ReadStream();
				string LogChannelName = Context.BuildInfo.ProjectName + "Test";
				List<string> TestLines = LogParser.GetLogFromChannel(LogChannelName, false).ToList();

				string LogCategory = "Log" + LogChannelName;
				string LogCategoryError = LogCategory + ": Error:";
				string LogCategoryWarning = LogCategory + ": Warning:";
				
				foreach (string Line in TestLines)
				{
					if (Line.StartsWith(LogCategoryError))
					{
						ReportError(Line);
					}
					else if (Line.StartsWith(LogCategoryWarning))
					{
						ReportWarning(Line);
					}
					else
					{
						Log.Info(Line);
					}
				}
			}

			base.TickTest();
		}

		/// <summary>
		/// This allows using a per-branch config to ignore certain issues
		/// that were inherited from Main and will be addressed there
		/// </summary>
		/// <param name="InArtifacts"></param>
		/// <returns></returns>
		protected override UnrealLog CreateLogSummaryFromArtifact(UnrealRoleArtifacts InArtifacts)
		{
			UnrealLog LogSummary = base.CreateLogSummaryFromArtifact(InArtifacts);

			IgnoredIssueConfig IgnoredIssues = new IgnoredIssueConfig();

			string IgnoredIssuePath = GetCachedConfiguration().IgnoredIssuesConfigAbsPath;

			if (!File.Exists(IgnoredIssuePath))
			{
				Log.Info("No IgnoredIssue Config found at {0}", IgnoredIssuePath);
			}
			else if (IgnoredIssues.LoadFromFile(IgnoredIssuePath))
			{
				Log.Info("Loaded IgnoredIssue config from {0}", IgnoredIssuePath);

				IEnumerable<UnrealLog.CallstackMessage> IgnoredEnsures = LogSummary.Ensures.Where(E => IgnoredIssues.IsEnsureIgnored(this.Name, E.Message));
				IEnumerable<UnrealLog.LogEntry> IgnoredWarnings = LogSummary.LogEntries.Where(E => E.Level == UnrealLog.LogLevel.Warning && IgnoredIssues.IsWarningIgnored(this.Name, E.Message));
				IEnumerable<UnrealLog.LogEntry> IgnoredErrors = LogSummary.LogEntries.Where(E => E.Level == UnrealLog.LogLevel.Error && IgnoredIssues.IsErrorIgnored(this.Name, E.Message));

				if (IgnoredEnsures.Any())
				{
					Log.Info("Ignoring {0} ensures.", IgnoredEnsures.Count());
					Log.Info("\t{0}", string.Join("\n\t", IgnoredEnsures.Select(E => E.Message)));
					LogSummary.Ensures = LogSummary.Ensures.Except(IgnoredEnsures).ToArray();
				}
				if (IgnoredWarnings.Any())
				{
					Log.Info("Ignoring {0} warnings.", IgnoredWarnings.Count());
					Log.Info("\t{0}", string.Join("\n\t", IgnoredWarnings.Select(E => E.Message)));
					LogSummary.LogEntries = LogSummary.LogEntries.Except(IgnoredWarnings).ToArray();
				}
				if (IgnoredErrors.Any())
				{
					Log.Info("Ignoring {0} errors.", IgnoredErrors.Count());
					Log.Info("\t{0}", string.Join("\n\t", IgnoredErrors.Select(E => E.Message)));
					LogSummary.LogEntries = LogSummary.LogEntries.Except(IgnoredErrors).ToArray();
				}
			}


			return LogSummary;
		}

		protected override UnrealProcessResult GetExitCodeAndReason(StopReason InReason, UnrealLog InLogSummary, UnrealRoleArtifacts InArtifacts, out string ExitReason, out int ExitCode)
		{
			// Check for login failure
			UnrealLogParser Parser = new UnrealLogParser(InArtifacts.AppInstance.GetLogReader());
			TConfigClass Config = GetCachedConfiguration();

			ExitReason = "";
			ExitCode = -1;

			foreach (HandledError ErrorToCheck in HandledErrors)
			{
				string[] MatchingErrors = Parser.GetErrors(ErrorToCheck.CategoryName).Where(E => E.Contains(ErrorToCheck.ClientErrorString)).ToArray();
				if (MatchingErrors.Length > 0)
				{
					ExitReason = string.Format("Test Error: {0} {1}", ErrorToCheck.GauntletErrorString, ErrorToCheck.Verbose ? "\"" + MatchingErrors[0] + "\"" : "");
					ExitCode = -1;
					return UnrealProcessResult.TestFailure;
				}
			}

			// If this is a Test Target Configuration and we have configured to ignore logging,
			// we can check the process exit code and return from here. This is especially useful
			// when logging is disabled in Test builds, but we still want to exit the test
			// successfully. 
			UnrealTargetConfiguration TargetConfig = InArtifacts.SessionRole.Configuration;
			bool bIgnoreLoggingInTest = Config.IgnoreTestBuildLogging && TargetConfig == UnrealTargetConfiguration.Test && !InLogSummary.EngineInitialized;
			bool bTestHasErrorLog = InLogSummary.FatalError != null || InLogSummary.HasTestExitCode;

			// This is a major assumption that Gauntlet captures the process exit code on all
			// platforms and is the only thing indicating if the test has actually passed or
			// failed in the absence of logging. 
			int ProcessExitCode = InArtifacts.AppInstance.ExitCode;
			if (InReason == StopReason.Completed && bIgnoreLoggingInTest && ProcessExitCode == 0)
			{
				ExitCode = 0;
				ExitReason = "Test build exited successfully without logs.";
				return UnrealProcessResult.ExitOk;
			}
			else if (InReason == StopReason.Completed && bIgnoreLoggingInTest && ProcessExitCode != 0 && !bTestHasErrorLog)
			{
				// Process has not exited cleanly and we do not have any error log messages. 
				// Fail the test here and provide context to user.
				ExitCode = ProcessExitCode;
				ExitReason = "Test build exited with error. Please enable logging in build for more information.";
				return UnrealProcessResult.TestFailure;
			}

			// Let the user know that if their tests are failing but process exit code is 0
			// and they have logging disabled in Test builds that they can ignore logging
			// failures if they wish to do so. 
			if(!InLogSummary.EngineInitialized && TargetConfig == UnrealTargetConfiguration.Test && ProcessExitCode == 0)
			{
				Log.Warning("*** Engine Initialization log not detected in Test build with Exit Code 0. " +
					"This test will fail. " +
					"Try passing `-AutomatedPerfTest.IgnoreTestBuildLogging` while running this test " +
					"or pass `-set:APTIgnoreTestBuildLogging=true` if running via BuildGraph if you " +
					"wish to ignore log parsing checks in this test or recompile with logging enabled. ***");
			}

			return base.GetExitCodeAndReason(InReason, InLogSummary, InArtifacts, out ExitReason, out ExitCode);
		}

		public override ITestReport CreateReport(TestResult Result, UnrealTestContext Context, UnrealBuildSource Build, IEnumerable<UnrealRoleResult> Artifacts, string ArtifactPath)
		{
			UnrealTargetPlatform Platform = Context.GetRoleContext(UnrealTargetRole.Client).Platform;
			TConfigClass Config = GetCachedConfiguration();
			string OutputPath = GetPerformanceReportArtifactOutputPath(Platform);
			HordeJobLink = Globals.Params.ParseValue("JobDetails", "");
			string DataSourceName = Config.DataSourceName;

			string PRSReportLink = "";

			// Always render reshape reports, failed tests may contain validation of interest
			if (Config.DoGPUReshape)
			{
				Config.WriteTestResultsForHorde = true;
				RenderGPUReshapeReport(ArtifactPath, OutputPath);
			}
			
			if (Result == TestResult.Passed)
			{
				string InsightsTraceFile = string.Empty;
				if (Config.DoInsightsTrace)
				{
					CopyInsightsTraceToOutput(ArtifactPath, OutputPath, out InsightsTraceFile);
				}
				
				if (GetCurrentPass() <= GetNumPasses() && Config.DoCSVProfiler)
				{
					// Our artifacts from each iteration such as the client log will be overwritten by subsequent iterations so we need to copy them out to another dir
					// to preserve them until we're ready to make our report on the final iteration.
					CopyPerfFilesToOutputDir(ArtifactPath, OutputPath);

					bool bGeneratedLocalReport = false;
					bool bGeneratedPRSReport = false;
					bool bGeneratedHordeTelemetry = false;

					// Local report generation is useful for people conducting tests locally without a centralized server (for A/B testing for instance).
					// To make it work out of the box, enable by default for non-build machine runs
					if (GenerateLocalReport)
					{
						// NOTE: This does not currently work with long paths due to the CsvTools not properly supporting them.
						Log.Info("Generating local performance reports using PerfReportTool.");
						bGeneratedLocalReport = GenerateLocalPerfReport(Platform, OutputPath);
					}

					// On build machines, default to producing PRS report
					if (GeneratePRSReport)
					{
						Dictionary<string, dynamic> CommonDataSourceFields = new Dictionary<string, dynamic>
						{
							{"HordeJobUrl", HordeJobLink}
						};
						
						Log.Info("Creating perf server importer with build name {BuildName}", OriginalBuildName);
						
						string ImportDirOverride = Globals.Params.ParseValue("PerfReportServerImportDir", null);
						Log.Info("Creating PRS Importer for data source '{0}' and import dir override (if any) '{1}'.",
							DataSourceName, ImportDirOverride);
						ICsvImporter Importer = ReportGenUtils.CreatePerfReportServerImporter(DataSourceName, OriginalBuildName,
							IsBuildMachine, ImportDirOverride, CommonDataSourceFields);

						if (Importer != null)
						{
							// Recursively grab all the csv files we copied to the temp dir and convert them to binary.
							List<FileInfo> AllBinaryCsvFiles = ReportGenUtils.CollectAndConvertCsvFilesToBinary(OutputPath);
							Log.Info("Found {0} valid binary csv files for import.", AllBinaryCsvFiles.Count);

							if (AllBinaryCsvFiles.Count == 0)
							{
								throw new AutomationException($"No Csv files found in {OutputPath}");
							}


							// The corresponding log for each csv sits in the same subdirectory as the csv file itself.
							IEnumerable<CsvImportEntry> ImportEntries = AllBinaryCsvFiles
								.Select(CsvFile => new CsvImportEntry(CsvFile.FullName, Path.Combine(CsvFile.Directory.FullName, "ClientOutput.log")));

							// todo update this so it associates videos with the correct CSVs
							IEnumerable<CsvImportEntry> CsvImportEntries = ImportEntries as CsvImportEntry[] ?? ImportEntries.ToArray();
							if (GetConfiguration().DoInsightsTrace)
							{
								string InsightsFilename = Path.GetFileNameWithoutExtension(CsvImportEntries.First().CsvFilename);

								InsightsFilename = string.IsNullOrEmpty(InsightsTraceFile) ? GetNormalizedInsightsFileName(InsightsFilename) : InsightsTraceFile + ".utrace";

								// recursively look for trace files that match the CSV's filename in the artifact path
								string[] MatchingTraces = FindFiles($"*{InsightsFilename}", true, ArtifactPath);

								if (MatchingTraces.Length > 0)
								{
									if (MatchingTraces.Length > 1)
									{
										Log.Warning("Multiple Insights traces were found in {ArtifactPath} matching pattern *{InsightsFilename}. Only the first will be attached to the CSV import for this test.",
											ArtifactPath, InsightsFilename);										
									}
									CsvImportEntries.First().AddAdditionalFile("Insights", MatchingTraces.First());
								}
								else
								{
									Log.Warning("Insights was requested, but no matching insights traces were found  matching pattern *{InsightsFilename} in {ArtifactPath}",
										InsightsFilename, ArtifactPath);
								}
							}

							if (GetConfiguration().DoVideoCapture)
							{
								string VideoPath = Path.Combine(ArtifactPath, "Client", "Videos");
								string[] VideoFiles = Directory.GetFiles(VideoPath, "*.mp4");
								if (VideoFiles.Length > 0)
								{
									foreach (var VideoFile in VideoFiles)
									{
										CsvImportEntries.First().AddAdditionalFile("Video", Path.Combine(VideoPath, VideoFile));
									}
								}
								else
								{
									Log.Warning("Video capture was requested, but no videos were found in path {VideoPath}", VideoPath);
								}
							}

							// Create the import batch
							Importer.Import(CsvImportEntries, out PRSReportLink);

							// trust blindly for now...
							bGeneratedPRSReport = true;
						}
						else
						{
							Log.Warning("Unable to create PRS Importer.");
						}

						// Cleanup the temp dir
						if(TempPerfCSVDir.Exists)
						{
							TempPerfCSVDir.Delete(recursive: true);
						}
					}

					if (GenerateHordeTelemetry)
					{
						string TestIdentity = CreateTestIdentity();

						bGeneratedHordeTelemetry = GenerateHordePerfReportTelemetry(Platform, TestIdentity, OutputPath);
					}

					if (!bGeneratedLocalReport && !bGeneratedPRSReport && !bGeneratedHordeTelemetry)
					{
						Log.Warning("Did not generate a local report, a PRS report, or emit Horde telemetry.");
					}
				}
			}
			else
			{
				Log.Warning("Skipping performance report generation because the perf report test failed.");
			}

			ITestReport Report = base.CreateReport(Result, Context, Build, Artifacts, ArtifactPath);

			if (Config.DoGPUReshape)
			{
				string ReportName = $"GPU Reshape - {Config.GPUReshapeWorkspace}";
				string FileFilter = "*GRS.Report.html";

				AddTestReportArtifacts(Result, Report, ReportName, OutputPath, FileFilter);
			}

			if (Config.DoInsightsTrace)
			{
				string UtraceLocation = Path.Combine(OutputPath, "Traces");
				string FileFilter = "*.utrace";
				string ArtifactReportName = "Trace Insights Report";

				AddTestReportArtifacts(Result, Report, ArtifactReportName, UtraceLocation, FileFilter);
			}

			if (!IsPRSAdded)
			{
				string PRSReportName = "PRS Report Upload";
				AddLinkToReport(Report, PRSReportName, PRSReportLink);
				IsPRSAdded = true;
			}

			if (Report != null)
			{
				Report.SetMetadata("PerfType", GetSubtestName(Config));
				if (!string.IsNullOrEmpty(Config.TestID))
				{
					if (GetCachedConfiguration().DoBuildZenStreaming)
					{
						Log.Info($"TestID: {Config.TestID}");
						Report.SetMetadata("PerfTest", "APTZen");
					}
					else
					{
						Report.SetMetadata("PerfTest", Config.TestID);
					}
				}
				else
				{
					Report.SetMetadata("PerfTest", "Default");
				}
				
			}

			return Report;
		}

		/// <inheritdoc/>
		protected override string HordeReportTestKey => base.HordeReportTestKey + (string.IsNullOrEmpty(GetCachedConfiguration().TestID) ? "" : $".{GetCachedConfiguration().TestID}");

		/// <summary>
		/// Gets the test identity of the <see cref="AutomatedPerfTestNode{TConfigClass}"/>.
		/// </summary>
		/// <returns>The test identity of the node.</returns>
		protected virtual string CreateTestIdentity(TConfigClass Config = null)
		{
			Config ??= GetCachedConfiguration();

			return Config.DataSourceName;
		}

		/// <summary>
		/// Get report type for current configuration
		/// </summary>
		/// <param name="ReportType"></param>
		/// <param name="SummaryTableType"></param>
		/// <param name="HistoricalReportType"></param>
		private void GetReportType(out string ReportType, out string SummaryTableType, out string HistoricalReportType)
		{
			ReportType = "ClientPerf";
			SummaryTableType = SummaryTable;
			HistoricalReportType = "autoPerfReportStandard";

			if (GetCachedConfiguration().DoLLM)
			{
				ReportType = "LLM";
				SummaryTableType = "autoPerfReportLlm";
				HistoricalReportType = "autoPerfReportLlm";
			}

			if(GetCachedConfiguration().DoBuildZenStreaming)
			{
				// Using defaults for now, but added here as a stub for incoming Zen Streaming Summary View.
			}
		}

		/// <summary>
		/// Produces a detailed csv report using PerfReportTool.
		/// Also, stores perf data in the perf cache, and generates a historic report using the data the cache contains.
		/// </summary>
		private bool GenerateLocalPerfReport(UnrealTargetPlatform Platform, string OutputPath)
		{
			string perfreportTool = "";
			perfreportTool = Path.Combine("PerfreportTool.dll");
	
			var ToolPath = FileReference.Combine(Unreal.EngineDirectory, "Binaries", "DotNET", "CsvTools", perfreportTool);
			if (!FileReference.Exists(ToolPath))
			{
				Log.Error("Failed to find perf report utility at this path: \"{ToolPath}\".", ToolPath);
				return false;
			}

			var ReportConfigDir = GetCachedConfiguration().ReportConfigDir;
			if (string.IsNullOrEmpty(ReportConfigDir))
			{
				// default to the report types and graphs provided by APT 
				ReportConfigDir = Path.Combine(Unreal.EngineDirectory.ToString(), "Plugins", "Performance", "AutomatedPerfTesting", "Build", "Scripts", "PerfReport");	
			}

			var ReportPath = GetCachedConfiguration().ReportPath;
			if(string.IsNullOrEmpty(ReportPath))
			{
				ReportPath = Path.Combine(OutputPath, "Reports");
			}

			string ReportCacheDir = Path.Combine(OutputPath, "Cache");

			// Csv files may have been output in one of two places.
			// Check both...
			var CsvsPaths = new[]
			{
				Path.Combine(OutputPath, "CSV")
			};

			var DiscoveredCsvs = new List<string>();
			foreach (var CsvsPath in CsvsPaths)
			{
				if (Directory.Exists(CsvsPath))
				{
					DiscoveredCsvs.AddRange(
						from CsvFile in Directory.GetFiles(CsvsPath, "*.csv", SearchOption.AllDirectories)
						select CsvFile);
				}
				// ->Warn if the directory doesn't exist
				else
				{
					Log.Warning($"Directory does not exist: {CsvsPath}");

				}
			}

			// ->If no csv files were found, we can't generate a report
			if (DiscoveredCsvs.Count == 0)
			{
				Log.Error("No CSV files found. Searched paths: {0}", string.Join(", ", CsvsPaths));
				if (Directory.Exists(OutputPath))
				{
					var allFiles = Directory.GetFiles(OutputPath, "*.*", SearchOption.AllDirectories);
					Log.Info($"All files in OutputPath ({OutputPath}): {string.Join(", ", allFiles)}");
				}
				else
				{
					Log.Warning($"OutputPath does not exist: {OutputPath}");
				}

				Log.Error("Test completed successfully but no csv profiling results were found. Searched paths were:\r\n  {Paths}", string.Join("\r\n  ", CsvsPaths.Select(s => $"\"{s}\"")));
				return false;
			}

			// Find the newest csv file and get its directory
			// (PerfReportTool will only output cached data in -csvdir mode)
			var NewestFile =
				(from CsvFile in DiscoveredCsvs
				 let Timestamp = File.GetCreationTimeUtc(CsvFile)
				 orderby Timestamp descending
				 select CsvFile).First();
			var NewestDir = Path.GetDirectoryName(NewestFile);

			Log.Info("Using perf report cache directory \"{ReportCacheDir}\".", ReportCacheDir);
			Log.Info("Using perf report output directory \"{ReportPath}\".", ReportPath);
			Log.Info("Using csv results directory \"{NewestDir}\". Generating historic perf report data...", NewestDir);

			// Make sure the cache and output directories exist
			if (!Directory.Exists(ReportCacheDir))
			{
				try { Directory.CreateDirectory(ReportCacheDir); }
				catch (Exception Ex)
				{
					Log.Error("Failed to create perf report cache directory \"{ReportCacheDir}\". {Ex}", ReportCacheDir, Ex);
					return false;
				}
			}
			if (!Directory.Exists(ReportPath))
			{
				try { Directory.CreateDirectory(ReportPath); }
				catch (Exception Ex)
				{
					Log.Error("Failed to create perf report output directory \"{ReportPath}\". {Ex}", ReportPath, Ex);
					return false;
				}
			}

			// Win64 is actually called "Windows" in csv profiles
			var PlatformNameFilter = Platform == UnrealTargetPlatform.Win64 ? "Windows" : $"{Platform}";

			string SearchPattern = $"{Context.BuildInfo.ProjectName}*";
			string ReportType = null;
			string SummaryTableType = null;
			string HistoricalSummaryType = null;

			GetReportType(out ReportType, out SummaryTableType, out HistoricalSummaryType);

			// Produce the detailed report, and update the perf cache
			var DetailedReportPath = Path.Combine(ReportPath, "Detailed");
			string[] CsvGenerationArgs = new[]
			{
				 $"\"{ToolPath.FullName}\"",
				 $"-csvdir \"{NewestDir}\"",
				 $"-reportType \"{ReportType}\"",
				 $"-o \"{DetailedReportPath}\"",
				 $"-reportxmlbasedir \"{ReportConfigDir}\"",
				 $"-summaryTable {SummaryTableType}",
				 $"-summaryTableCache \"{ReportCacheDir}\"",
				 $"-metadatafilter platform=\"{PlatformNameFilter}\""
			};

			string PerfReportToolArgs = string.Join(' ', CsvGenerationArgs);
			RunAndLog(CmdEnv, CmdEnv.DotnetMsbuildPath, PerfReportToolArgs, out int ErrorCode);
			if (ErrorCode != 0)
			{
				Log.Error("PerfReportTool returned error code \"{ErrorCode}\" while generating detailed report.", ErrorCode);
			}

			// Now generate the all-time historic summary report
			HistoricReport("HistoricReport_AllTime", new[]
			{
				$"platform={PlatformNameFilter}"
			});

			// 14 days historic report
			HistoricReport($"HistoricReport_14Days", new[]
			{
				$"platform={PlatformNameFilter}",
				$"starttimestamp>={DateTimeOffset.Now.ToUnixTimeSeconds() - (14 * 60L * 60L * 24L)}"
			});

			// 7 days historic report
			HistoricReport($"HistoricReport_7Days", new[]
			{
				$"platform={PlatformNameFilter}",
				$"starttimestamp>={DateTimeOffset.Now.ToUnixTimeSeconds() - (7 * 60L * 60L * 24L)}"
			});

			void HistoricReport(string Name, IEnumerable<string> Filter)
			{
				var Args = new[]
				{
					$"{ToolPath.FullName}",
					$"-reportType \"{ReportType}\"",
					$"-summarytablecachein \"{ReportCacheDir}\"",
					$"-summaryTableFilename \"{Name}.html\"",
					$"-reportxmlbasedir \"{ReportConfigDir}\"",
					$"-o \"{ReportPath}\"",
					$"-metadatafilter \"{string.Join(" and ", Filter)}\"",
					$"-summaryTable {HistoricalSummaryType}",
					$"-condensedSummaryTable {HistoricalSummaryType}",
					$"-reportLinkRootPath \"{DetailedReportPath}\\\"",
					"-emailtable",
					"-recurse"
				};

				var ArgStr = string.Join(" ", Args);

				RunAndLog(CmdEnv, CmdEnv.DotnetMsbuildPath, ArgStr, out ErrorCode);
				if (ErrorCode != 0)
				{
					Log.Error("PerfReportTool returned error code \"{ErrorCode}\" while generating historic report.", ErrorCode);
				}
			}

			return true;
		}

		/// <summary>
		/// Emits performance trend telemetry to Horde via Performance Report Tool.
		/// </summary>
		/// <param name="Platform">The platform to emit telemetry for.</param>
		/// <param name="TestIdentity">The test identity of the telemetry.</param>
		/// <param name="OutputPath">The output path of the csv data.</param>
		/// <param name="SummaryTable">The summary table to emit data for. Defaults to KeyStats.</param>
		/// <returns>True if performance telemetry ran successfully, false otherwise.</returns>
		private bool GenerateHordePerfReportTelemetry(UnrealTargetPlatform Platform, string TestIdentity, string OutputPath, string SummaryTable = "KeyStats")
		{
			var ReportConfigDir = GetCachedConfiguration().ReportConfigDir;
			if (string.IsNullOrEmpty(ReportConfigDir))
			{
				// default to the report types and graphs provided by APT 
				ReportConfigDir = Path.Combine(Unreal.EngineDirectory.ToString(), "Plugins", "Performance", "AutomatedPerfTesting", "Build", "Scripts", "PerfReport");
			}

			var DiscoveredCsvs = new List<string>();

			var CsvsPaths = new[]
			{
				Path.Combine(OutputPath, "CSV")
			};

			foreach (var CsvsPath in CsvsPaths)
			{
				if (!Directory.Exists(CsvsPath))
				{
					Log.Warning($"Directory does not exist: {CsvsPath}");
					continue;
				}

				DiscoveredCsvs.AddRange(Directory.GetFiles(CsvsPath, "*.csv.bin", SearchOption.AllDirectories));
			}

			if (DiscoveredCsvs.Count == 0)
			{
				Log.Info($"No .csv.bin files found, attempting CSV -> CSV.BIN conversion on {OutputPath}...");

				List<FileInfo> GeneratedCsvBins = ReportGenUtils.CollectAndConvertCsvFilesToBinary(OutputPath);

				if (GeneratedCsvBins.Count == 0)
				{
					Log.Error($"No Csv files found in {OutputPath}");

					return false;
				}

				DiscoveredCsvs.AddRange(from csvBin in GeneratedCsvBins select csvBin.FullName);
			}

			// Find the newest csv file and get its directory
			// (PerfReportTool will only output cached data in -csvdir mode)
			var NewestFile =
				(from CsvFile in DiscoveredCsvs
				 let Timestamp = File.GetCreationTimeUtc(CsvFile)
				 orderby Timestamp descending
				 select CsvFile).First();
			var NewestDir = Path.GetDirectoryName(NewestFile);

			var PlatformNameFilter = Platform == UnrealTargetPlatform.Win64 ? "Windows" : $"{Platform}";
			
			string ReportType = null;
			GetReportType(out ReportType, out _, out _);
			Log.Info("Horde Telemetry Emission detected the following report type: {0}", ReportType);

			if(ReportType != "ClientPerf" && SummaryTable == "KeyStats")
			{
				Log.Info("Horde Telemetry is currently not possible for events outside of ClientPerf reports (when using KeyStats summary table type) - skipping emissions.");

				return true;
			}

			string[] CsvGenerationArgs = new[]
			{
				 $"-csvdir \"{NewestDir}\"",
				 $"-reportType \"{ReportType}\"",
				 $"-summaryTable {SummaryTable}",
				 $"-metadatafilter platform=\"{PlatformNameFilter}\"",
				 "-nodetailedreports",
				 "-csvTable",
				 "-summaryTableDataToTelemetry",
				 $"-telemetryTestIdentity \"{TestIdentity}\""
			};

			string PerfReportToolArgs = string.Join(' ', CsvGenerationArgs);
			int ErrorCode = ReportGenUtils.RunPerfReportTool(PerfReportToolArgs, ReportConfigDir);

			if (ErrorCode != 0)
			{
				Log.Error($"PerfReportTool returned error code \"{ErrorCode}\" while generating detailed report.");
				return false;
			}

			return true;
		}

		private void CopyPerfFilesToOutputDir(string FromArtifactPath, string ToOutputDir = null)
		{
			if(string.IsNullOrEmpty(ToOutputDir))
			{
				// Fallback path
				ToOutputDir = TempPerfCSVDir.FullName;
			}

			DirectoryInfo OutputDirectory = new DirectoryInfo(ToOutputDir);
			if (!OutputDirectory.Exists)
			{
				Log.Info("Creating temp perf csv dir: {OutputDirectory}", OutputDirectory);
				OutputDirectory.Create();
			}

			DirectoryInfo CSVDirectory = new DirectoryInfo(Path.Combine(ToOutputDir, "CSV"));
			if (!CSVDirectory.Exists)
			{
				Log.Info($"Creating CSV Directory: {CSVDirectory}");
				CSVDirectory.Create();
			}

			// ->Log to check if dirs exist
			Log.Info($"OutputDirectory exists: {OutputDirectory.Exists}, CSVDirectory exists: {CSVDirectory.Exists}");

			string ClientArtifactDir = Path.Combine(FromArtifactPath, "Client");
			string ClientLogPath = Path.Combine(ClientArtifactDir, "ClientOutput.log");


			string CSVPath = PathUtils.FindRelevantPath(ClientArtifactDir, "Profiling", "CSV");
			if (string.IsNullOrEmpty(CSVPath))
			{
				Log.Warning("Failed to find CSV folder folder in {ClientArtifactDir}", ClientArtifactDir);
				return;
			}

			FileInfo LogFileInfo = new FileInfo(ClientLogPath);
			if (LogFileInfo.Exists)
			{
				// Create a subdir for each pass as we want to store the csv and log together in the same dir to make it easier to find them later.
				string PassDir = Path.Combine(ToOutputDir, "Logs");
				Directory.CreateDirectory(PassDir);

				string Guid = TestGuid.ToString();
				string DestLogFile = $"CL{Context.BuildInfo.Changelist}-Pass{GetCurrentPass()}-{Guid.Substring(0, Guid.IndexOf("-"))}-{LogFileInfo.Name}";
				string LogDestPath = Path.Combine(PassDir, DestLogFile);
				Log.Info("Copying Log {ClientLogPath} To {LogDest}", ClientLogPath, LogDestPath);
				CommandUtils.CopyFile(ClientLogPath, LogDestPath, false, true);
			}
			else
			{
				Log.Warning("No log file was found at {ClientLogPath}", ClientLogPath);
			}

			// Grab all the csv files that have valid metadata.
			// We don't want to convert to binary in place as the legacy reports require the raw csv.
			List<FileInfo> CsvFiles = ReportGenUtils.CollectValidCsvFiles(CSVPath);
			if (!CopyCSVs(CsvFiles, CSVDirectory))
			{
				Log.Warning("No valid csv files found in {CSVPath}", CSVPath);
			}
		}

		protected virtual bool CopyCSVs(List<FileInfo> CsvFiles, DirectoryInfo CSVDirectory)
		{
			if (CsvFiles.Count > 0)
			{
				// We only want to copy the latest file as the other will have
				// already been copied when this was run for those iterations.
				CsvFiles.SortBy(Info => Info.LastWriteTimeUtc);
				FileInfo LatestCsvFile = CsvFiles.Last();
				CopyCSV(LatestCsvFile, CSVDirectory);
				return true;
			}

			return false;
		}

		protected void CopyCSV(FileInfo CSVFile, DirectoryInfo CSVDirectory)
		{
			string Extension = CSVFile.Extension;
			string OutputCSVFile = $"{CSVFile.Name.Replace(Extension, "")}-Pass{GetCurrentPass()}{Extension}";
			string CsvDestPath = Path.Combine(CSVDirectory.FullName, OutputCSVFile);
			Log.Info("Copying Csv {CsvPath} To {CsvDestPath}", CSVFile.FullName, CsvDestPath);
			//CSVFile.CopyTo(CsvDestPath, true);
			CommandUtils.CopyFile(CSVFile.FullName, CsvDestPath, false, true);
		}
		
		protected virtual string GetSubtestName(TConfigClass Config)
		{
			// Options like DoLLM and DoInsightsTrace are heavy enough to be in their own subtest type
			// Since they are not exclusive, create a yet another subtest type if both are specified
			if (Config.DoLLM && Config.DoInsightsTrace)
			{
				throw new AutomationException($"Running Insights trace with LLM is not a practical test due to LLM's large overhead.");
			}

			if (Config.DoInsightsTrace)
			{
				const string DefaultTraceChannels = "default,screenshot,stats"; 
				if (Config.TraceChannels.Trim() != DefaultTraceChannels && GeneratePRSReport)
				{
					Log.Warning($"Running Insights trace with non-default channels {Config.TraceChannels}. If this was not intended, it might contaminate 'Insights' subtest results." +
						$" Change default trace channels to avoid this warning.");
				}

				return "Insights";
			}

			if (Config.DoLLM)
			{
				return "LLM";
			}

			if (Config.DoGPUPerf)
			{
				return "GPUPerf";
			}

			if (Config.DoGPUReshape)
			{
				return "GPUReshape";
			}

			return "Perf";
		}

		public override TConfigClass GetConfiguration()
		{
			TConfigClass Config = base.GetConfiguration();
			Config.MaxDuration = Context.TestParams.ParseValue("MaxDuration", 60 * 60);  // 1 hour max

			if (string.IsNullOrEmpty(Config.PerfOutputPath))
			{
				Config.PerfOutputPath = Path.Combine(Context.BuildInfo.ProjectPath.Directory.FullName, "Saved", "Performance");
			}

			BaseOutputPath = Path.Combine(Config.PerfOutputPath, GetType().Name); 

			UnrealTestRole ClientRole = Config.RequireRole(UnrealTargetRole.Client);
			// the controller will be added by the subclasses

			ClientRole.CommandLineParams.AddOrAppendParamValue("logcmds", "LogHttp Verbose, LogAutomatedPerfTest Verbose");

			// force the screen to stay awake during the test
			ClientRole.CommandLineParams.Add("keepscreenawake");
			
			if(Config.Deterministic)
			{
				ClientRole.CommandLineParams.Add("deterministic");
			}

			Log.Info("AutomatedPerfTestNode<>.GetConfiguration(): Config.DoFPSChart={0}, Config.DoCSVProfiler={1}, Config.DoVideoCapture={2}, Config.DoInsightsTrace={3}, Config.DoLLM={4}", 
				Config.DoFPSChart, Config.DoCSVProfiler, Config.DoVideoCapture, Config.DoInsightsTrace, Config.DoLLM);

			if (Config.DoBuildZenStreaming)
			{
				Config.TestID = "APTZen";
				UnrealTargetPlatform ClientPlatform = Context.GetRoleContext(UnrealTargetRole.Client).Platform;

				if (Config.DoLocalZenTest)
                {
					// Skip Android, empty ZenStoreHost auto-resolves to localhost, allowing AndroidPortForwarder plugin tunnels via adb reverse.
					if(ClientPlatform != UnrealTargetPlatform.Android)
					{
						Config.ZenStoreHost = UnrealHelpers.GetHostIpAddress();	
					}					
                }

				// Add Zen Streaming params to Client command.
				ClientRole.CommandLineParams.Add("DoBuildZenStreaming");
				ClientRole.CommandLineParams.AddOrAppendParamValue("ZenStoreHost", Config.ZenStoreHost);
				ClientRole.CommandLineParams.AddOrAppendParamValue("ZenStorePort", Config.ZenStorePort);
				ClientRole.CommandLineParams.AddOrAppendParamValue("ZenStoreProject", Config.ZenStoreProject);
				ClientRole.CommandLineParams.AddOrAppendParamValue("ZenStorePlatform", Config.ZenStorePlatform);
			}
			ClientRole.CommandLineParams.AddOrAppendParamValue("AutomatedPerfTest.TestID", Config.TestID);

			if (Config.DoInsightsTrace)
			{
				ClientRole.CommandLineParams.Add("statnamedevents");
				ClientRole.CommandLineParams.Add("AutomatedPerfTest.DoInsightsTrace");
				if (Config.TraceChannels != String.Empty)
				{
					ClientRole.CommandLineParams.AddOrAppendParamValue("AutomatedPerfTest.TraceChannels", Config.TraceChannels);
				}
			}

			if (Config.DoLLM)
			{
				ClientRole.CommandLineParams.Add("llm");
				ClientRole.CommandLineParams.Add("llmcsv");
			}

			if (Config.WaitForAttach)
			{
				ClientRole.CommandLineParams.Add("waitforattach");
			}

			if (!String.IsNullOrEmpty(Config.NullOSSPlatforms) && Config.NullOSSPlatforms.Split("+").Contains(Context.GetRoleContext(UnrealTargetRole.Client).Platform.ToString()))
			{
				ClientRole.CommandLineParams.Add("ini:Engine:[OnlineSubsystem]:DefaultPlatformService=Null");
			}

			if (Config.DoGPUPerf)
			{
				// see ReplayRun.py, reducedAsyncComputeCommands
				// We enable r.nanite.asyncrasterization.shadowdepths because nanite overlaps with itself, so it's possible to time accurately without distorting other timings
				ClientRole.CommandLineParams.AddOrAppendParamValue("execcmds", "r.StencilLODMode 1");
				ClientRole.CommandLineParams.AddOrAppendParamValue("execcmds", "r.VolumetricRenderTarget.PreferAsyncCompute 0");
				ClientRole.CommandLineParams.AddOrAppendParamValue("execcmds", "r.LumenScene.Lighting.AsyncCompute 0");
				ClientRole.CommandLineParams.AddOrAppendParamValue("execcmds", "r.Lumen.DiffuseIndirect.AsyncCompute 0");
				ClientRole.CommandLineParams.AddOrAppendParamValue("execcmds", "r.Bloom.AsyncCompute 0");
				ClientRole.CommandLineParams.AddOrAppendParamValue("execcmds", "r.nanite.asyncrasterization.shadowdepths 1");
				ClientRole.CommandLineParams.AddOrAppendParamValue("execcmds", "r.TSR.AsyncCompute 0");
				ClientRole.CommandLineParams.AddOrAppendParamValue("execcmds", "r.RayTracing.AsyncBuild 0");
				ClientRole.CommandLineParams.AddOrAppendParamValue("execcmds", "r.DFShadowAsyncCompute 0");
				ClientRole.CommandLineParams.AddOrAppendParamValue("execcmds", "r.AmbientOcclusion.Compute 1"); // 1 here means compute on the graphics pipe
				ClientRole.CommandLineParams.AddOrAppendParamValue("execcmds", "r.LocalFogVolume.TileCullingUseAsync 0");
				ClientRole.CommandLineParams.AddOrAppendParamValue("execcmds", "r.SkyAtmosphereASyncCompute 0");
				ClientRole.CommandLineParams.AddOrAppendParamValue("execcmds", "r.Substrate.AsyncClassification 0");
			}

			if (Config.DoGPUPerf || Config.LockDynamicRes)
			{
				ClientRole.CommandLineParams.Add("AutomatedPerfTest.LockDynamicRes");
			}

			if(Config.DoFPSChart)
			{
				ClientRole.CommandLineParams.Add("AutomatedPerfTest.DoFPSChart");
			}

			if (Config.DoCSVProfiler)
			{
				ConfigureCSVProfiler(Config, ClientRole);
			}

			if (Config.DoVideoCapture)
			{
				ClientRole.CommandLineParams.Add("AutomatedPerfTest.DoVideoCapture");
			}

			if(Config.WaitForDebugger)
			{
				// Useful if you want to attach the debugger on the client
				ClientRole.CommandLineParams.Add("WaitForDebugger");
			}

			// Setting default to true as tests which do not require
			// server role do not need MCP.
			Config.NoMCP = true;

			// Get config bridge if specified. This is useful if we want project
			// specific configuration which can be injected via params 
			if (!string.IsNullOrEmpty(Config.ConfigBridges))
			{
				AutoTestBridge.Initialize(Config.ConfigBridges.Split(","));
			}

			AutoTestBridge.Configure(Context, Config);
			AutoTestBridge.ConfigureClient(ClientRole, Context, Config);

			if (AutoTestBridge.IsServerRoleRequired())
			{
				// Override default if server role is required. 
				Config.NoMCP = Context.TestParams.ParseParam("NoMCP");
				string DefaultClientSenderId = $"{Context.BuildInfo.ProjectName}Client";
				ConfigureServerRole(Config);

				// Some fun necessary commandlines to make sure our sockets are working properly. 
				string BindAddress = Context.TestParams.ParseValue("bindaddress", "any");
				ClientRole.CommandLineParams.Add("ini:Engine:[HTTPServer.Listeners]:DefaultBindAddress", BindAddress);
				ClientRole.CommandLineParams.Add("ini:Engine:[HTTPServer.Listeners]:DefaultReuseAddressAndPort", true);
				ClientRole.CommandLineParams.Add("logcmds", "LogSockets VeryVerbose");

				// Tell client and server where to send messages to communicate with Gauntlet, and give them standardized names.
				ClientRole.CommandLineParams.AddUnique("externalrpclistenaddress", RpcExecutor.GetListenAddressAndPort());
				ClientRole.CommandLineParams.AddUnique("rpcsenderid", DefaultClientSenderId);
			}

			if (!string.IsNullOrEmpty(Config.ManagedAcountPool))
			{
				OverrideManagedAccountPool(Config.ManagedAcountPool);
			}

			if(!IsBuildMachine && Config.OverrideArtifactOutputPath)
			{
				// If we are on a build machine, artifacts are stored and handled as expected. On build machines, reports
				// are eventually copied to this location before being processed. When generating local reports, it is
				// possible artifacts are being output to a place not expected by APT, so we let the runtime know to 
				// output the artifacts to this location directly if required. 

				UnrealTargetPlatform Platform = Context.GetRoleContext(UnrealTargetRole.Client).Platform;
				if (PlatformTargetSupport.IsHostMountingSupported(Platform))
				{
					string ArtifactOutputPath = GetPerformanceReportArtifactOutputPath(Platform, Config);
					ClientRole.CommandLineParams.AddUnique("AutomatedPerfTest.ArtifactOutputPath", PlatformTargetSupport.GetHostMountedPath(Platform, ArtifactOutputPath));
				}
			}

			ClientRole.ArchiveDevArtifacts = ForceArchiveDevArtifacts;

			if (!string.IsNullOrEmpty(Config.CVars))
			{
				string[] Cvars = Config.CVars.Split(',');
				foreach (string Cvar in Cvars)
				{
					ClientRole.CommandLineParams.AddOrAppendParamValue("dpcvars", Cvar.Trim());
				}
			}

			return Config;
		}

		private void RenderGPUReshapeReport(string ArtifactPath, string OutputPath)
		{
			InternalUtils.SafeCreateDirectory(OutputPath, true);
			
			// Hardcoded path, for now
			// Raytracing serves as the current development branch
			string GPUReshapeBranch = "Raytracing";
			string GPUReshapePath   = Path.Combine(Unreal.RootDirectory.FullName, $"Engine/Binaries/ThirdParty/GPUReshape/Win64/{GPUReshapeBranch}/GPUReshape.exe");
			
			// Get all reports, search each test case
			var Reports = Directory.GetFiles(ArtifactPath, "*GRS.Report.json", SearchOption.AllDirectories).ToList();
			if (Reports.Count == 0)
			{
				Log.Error($"No GPU-Reshape reports found in '{ArtifactPath}'");
				return;
			}

			foreach (string ReportPath in Reports)
			{
				// Default to HTML report, for now
				string RenderName = Path.GetFileNameWithoutExtension(ReportPath);
				string RenderPath = Path.Combine(OutputPath, $"{RenderName}.html");

				// Setup arguemnts
				StringBuilder Arguments = new();
				Arguments.Append("render");
				Arguments.Append($" -report=\"{ReportPath}\"");
				Arguments.Append($" -out=\"{RenderPath}\"");

				// Render the report
				Log.Info($"Rendering GPUReshape ('{GPUReshapePath}') report with arguments `{Arguments}`");
				if (Process.Start(GPUReshapePath, Arguments.ToString()) is not {} process)
				{
					Log.Error("Failed to start render process");
					continue;
				}
				
				process.WaitForExit();
			}
		}

		private void AddTestReportArtifacts(TestResult Result, ITestReport Report, string ArtifactName, string OutputPath, string FileFilter)
		{
			// Attach Report Info
			if (Report is AutomatedTestSessionData SessionReport)
			{
				TestEventStream Stream = SessionReport.GetEventStream();
				string[] FoundFiles = FindFiles(FileFilter, true, OutputPath);
				List<string> HTMLFiles = FoundFiles.Where(F => F.EndsWith(".html", StringComparison.OrdinalIgnoreCase)).ToList();
				List<string> FilteredFiles = FoundFiles.Except(HTMLFiles).ToList();
				if (HTMLFiles.Any())
				{
					foreach (string AbsoluteFilePath in HTMLFiles)
					{
						string EntryName = Path.GetFileName(AbsoluteFilePath);
						string RelativePath = Path.GetRelativePath(OutputPath, AbsoluteFilePath);
						EmbeddedHTMLFile EmbeddedFile = new EmbeddedHTMLFile(EntryName, RelativePath, OutputPath);
						Stream.AddArtifact(DateTime.UtcNow, Microsoft.Extensions.Logging.LogLevel.Information, $"{{{EmbeddedFile.Token}}}", EmbeddedFile);
					}
				}
				if (FilteredFiles.Any())
				{
					Dictionary<string, string> Files = new Dictionary<string, string>();
					foreach (string AbsoluteFilePath in FilteredFiles)
					{
						string EntryName = Path.GetFileName(AbsoluteFilePath);
						string RelativePath = Path.GetRelativePath(OutputPath, AbsoluteFilePath);
						Files.Add(EntryName, RelativePath);
					}
					ArtifactFiles ArtifactFiles = new ArtifactFiles(ArtifactName, Files, OutputPath);
					Stream.AddArtifact(DateTime.UtcNow, Microsoft.Extensions.Logging.LogLevel.Information, $"{{{ArtifactFiles.Token}}}", ArtifactFiles);
				}
			}
			else if (Report != null)
			{
				foreach (string AbsoluteFilePath in FindFiles(FileFilter, true, OutputPath))
				{
					string BaseFileName = Path.GetFileName(AbsoluteFilePath);
					Report.AttachArtifact(AbsoluteFilePath, BaseFileName);
				}
			}
			else
			{
				Log.Warning("Test Report is null");
			}
		}

		public void CopyInsightsTraceToOutput(string FromArtifactPath, string ToOutputPath, out string OutTraceName)
		{
			OutTraceName = string.Empty;
			Log.Info("Copying test insights trace from artifact path to report cache");
			
			// find all the available trace paths
			var DiscoveredTraces = new List<string>();
			if (Directory.Exists(FromArtifactPath))
			{
				DiscoveredTraces.AddRange(
					from TraceFile in Directory.GetFiles(FromArtifactPath, "*.utrace", SearchOption.AllDirectories)
					select TraceFile);
			}
			
			// if we couldn't find any traces, report that and bail out
			if (DiscoveredTraces.Count == 0)
			{
				Log.Error("Test completed successfully but no trace results were found. Searched path was {ArtifactPath}", FromArtifactPath);
				return;
			}
			
			// iterate over each of the discovered traces (there should be one for each test case that was run)
			// first, sort the cases by timestamp
			string[] SortedTraces =
				(from TraceFile in DiscoveredTraces
					let Timestamp = File.GetCreationTimeUtc(TraceFile)
					orderby Timestamp descending
					select TraceFile).ToArray();
			
			var ReportPath = Path.Combine(ToOutputPath, "Traces");
			if (SortedTraces.Length > 0)
			{
				string Filename = Path.GetFileNameWithoutExtension(SortedTraces[0]);
				string PerfTracePath = Path.Combine(ReportPath, Filename + ".utrace");

				Log.Info("Copying latest utrace file from {ArtifactPath} to Perf .utrace path: {PerfTracePath}", FromArtifactPath,
					PerfTracePath);
				
				// just try the copy over, and log a failure, but don't bail out of the test.
				try
				{
					InternalUtils.SafeCreateDirectory(Path.GetDirectoryName(PerfTracePath), true);
					File.Copy(SortedTraces[0], PerfTracePath);
					OutTraceName = Filename;
				}
				catch (Exception e)
				{
					Log.Warning("Failed to copy local trace file: {Text}", e);
				}
			}
		}

		protected void ConfigureCSVProfiler(TConfigClass Config, UnrealTestRole ClientRole)
		{
			if (Globals.Params.ParseParam("LocalReports") || (!IsBuildMachine && !Globals.Params.ParseParam("NoLocalReports")))
			{
				GenerateLocalReport = true;
			}

			if((IsBuildMachine || Globals.Params.ParseParam("PerfReportServer")) &&
						!Globals.Params.ParseParam("SkipPerfReportServer"))
			{
				GeneratePRSReport = true;
			}

			if(IsBuildMachine && Globals.Params.ParseParam("HordeTelemetry"))
			{
				GenerateHordeTelemetry = true;
			}

			ClientRole.CommandLineParams.Add("AutomatedPerfTest.DoCSVProfiler");
			ClientRole.CommandLineParams.Add("csvGpuStats");

			if(string.IsNullOrEmpty(Config.TestBuildVersion))
			{
				Config.TestBuildVersion = OriginalBuildName;
			}

			if(string.IsNullOrEmpty(Config.BuildVersion))
			{
				// If build version has been overridden for any reason,
				// this ensures that the CSV metadata matches the test
				// build version since this var is used while instantiating
				// perf report importer. 
				Config.BuildVersion = OriginalBuildName;
			}

			OriginalBuildName = Config.BuildVersion;

			Log.Info($"Using Test Build Version {Config.TestBuildVersion}");

			// Add CSV metadata
			List<string> CsvMetadata =
			[
				$"testname={Context.BuildInfo.ProjectName}",
				"gauntletTestType=AutomatedPerfTest",
				$"gauntletSubTest={GetSubtestName(Config)}",
				"testBuildIsPreflight=" + (ReportGenUtils.IsTestingPreflightBuild(Config.TestBuildVersion) ? "1" : "0"),
				$"testBuildVersion={Config.TestBuildVersion}",
				"testconfigname=" + Config.TestConfigName,
			];

			if (!string.IsNullOrEmpty(Config.TestConfigName))
			{
				CsvMetadata.Add("deviceconfigoverride=" + Config.TestConfigName);
			}

			if (!string.IsNullOrEmpty(Context.BuildInfo.Branch) && Context.BuildInfo.Changelist != 0)
			{
				CsvMetadata.Add("branch=" + Context.BuildInfo.Branch);
				CsvMetadata.Add("changelist=" + Context.BuildInfo.Changelist);
			}

			if (Config.DoGPUPerf)
			{
				CsvMetadata.Add("ReducedAsyncCompute=1");
			}

			ClientRole.CommandLineParams.Add("AutomatedPerfTest.CsvMetadata", "\"" + String.Join(",", CsvMetadata) + "\"");
		}

		private void ConfigureServerRole(TConfigClass Config)
		{
			// Assumed default base port. TODO: Should be overridable. 
			const int DefaultBasePort = 12321;
			string DefaultServerSenderId = $"{Context.BuildInfo.ProjectName}Server";
			UnrealTestRole ServerRole = Config.RequireRole(UnrealTargetRole.Server);

			ServerRole.CommandLineParams.AddUnique("rpcport", DefaultBasePort + 1);
			ServerRole.CommandLineParams.AddUnique("externalrpclistenaddress", RpcExecutor.GetListenAddressAndPort());
			ServerRole.CommandLineParams.AddUnique("rpcsenderid", DefaultServerSenderId);

			AutoTestBridge.ConfigureServer(ServerRole, Context, Config);
		}

		private static void OverrideManagedAccountPool(string ManagedAccountPool)
		{
			if (!string.IsNullOrEmpty(ManagedAccountPool))
			{
				Type ManagedAccountPoolType = Util.GetTypeWithInterface<IManagedAccountPool>(ManagedAccountPool);
				MethodInfo InitGenericMethod = typeof(AccountPool).GetMethods(BindingFlags.Static | BindingFlags.Public)
					.Where(Info => Info.Name == "Initialize" && Info.IsGenericMethod)
					.First();

				// Invokes Initialize<T>() with given Managed Account Pool implementation
				// type which will override the existing instance, if any.
				MethodInfo InitMethodRef = InitGenericMethod.MakeGenericMethod(ManagedAccountPoolType);
				InitMethodRef.Invoke(null, null);
			}
		}

		static protected void GetConfigValues(UnrealTestContext Context, string IniSection, string IniElement, out IReadOnlyList<string> Values)
		{
			IniConfigUtil.GetConfigHierarchy(Context, ConfigHierarchyType.Engine).TryGetValues(IniSection, IniElement, out Values);
		}

		static protected string GetPathInProject(UnrealTestContext Context, string InPath)
		{
			return Path.Combine(Context.BuildInfo.ProjectPath.Directory.FullName, InPath);
		}

		static protected void ReadConfigArray(UnrealTestContext Context, string IniSection, string IniElement, Action<string> Process)
		{
			IReadOnlyList<string> Configs = null;
			GetConfigValues(Context, IniSection, IniElement, out Configs);
			List<string> ConfigList = Configs == null ? new List<string>() : Configs.ToList();
			ConfigList.ForEach(Process);
		}
	}

	public class DefaultTest : AutomatedPerfTestNode<AutomatedPerfTestConfigBase>
	{
		public DefaultTest(UnrealTestContext InContext) 
			: base(InContext)
		{
			InitDefaultTestTypes();
		}

		private ITestNode CreateDefaultTestType(out Type TestNodeType)
		{
			IAutomatedPerfTest TestNode = null;
			TestNodeType = null;

			foreach (Type TestType in CandidateTestTypes)
			{
				try
				{
					ConstructorInfo Constructor = TestType.GetConstructor([typeof(UnrealTestContext)]);
					TestNode = Constructor?.Invoke([Context]) as IAutomatedPerfTest;
				}
				catch (Exception)
				{
					continue;
				}

				// Return the first candidate test type which have test(s) configured in project settings
				TestNodeType = TestType;
				List<string> Tests = TestNode.GetTestsFromConfig();
				if (Tests != null && Tests.Count > 0)
				{
					// Once we have found the first compatible test, bail out. 
					break;
				}
			}

			return TestNode as ITestNode;
		}

		public void InitDefaultTestTypes()
		{
			CandidateTestTypes = Gauntlet.Utils.InterfaceHelpers.FindTypes<IAutomatedPerfTest>(true, bConcreteTypesOnly:true).ToHashSet();
		}

		public override AutomatedPerfTestConfigBase GetConfiguration()
		{
			if (CachedConfig != null) 
			{
				return CachedConfig;
			}

			Type TestType;
			ITestNode TestNode = CreateDefaultTestType(out TestType);
			if(TestNode == null)
			{
				throw new AutomationException("Could not find a default test for given project. " +
					"Configure one of the available Automated Perf Tests in settings before re-running this test.");
			}

			dynamic TestNodeObject = Convert.ChangeType(TestNode, TestType);
			AutomatedPerfTestConfigBase Config = TestNodeObject?.GetConfiguration();
			
			// Let Default Test enable CSV profiler by default. 
			Config.DoCSVProfiler = true;
			ConfigureCSVProfiler(Config, Config.RequireRole(UnrealTargetRole.Client)); 

			// Pull all info we need from derived test node. 
			CachedConfig = Config;
			BaseOutputPath = TestNodeObject?.BaseOutputPath;
			return Config;
		}

		private HashSet<Type> CandidateTestTypes = new HashSet<Type>(); 
	}
}
