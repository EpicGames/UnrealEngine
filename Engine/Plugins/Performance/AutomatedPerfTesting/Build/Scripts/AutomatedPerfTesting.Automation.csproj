<Project Sdk="Microsoft.NET.Sdk">
  <Import Project="AutomatedPerfTesting.Automation.csproj.props" Condition="Exists('AutomatedPerfTesting.Automation.csproj.props')" />
  <Import Project="..\..\..\..\..\Source\Programs\Shared\UnrealEngine.csproj.props" />
  <PropertyGroup>
    <OutputType>Library</OutputType>
    <GenerateTargetFrameworkAttribute>false</GenerateTargetFrameworkAttribute>
    <RootNamespace>AutomatedPerfTesting.Automation</RootNamespace>
    <AssemblyName>AutomatedPerfTesting.Automation</AssemblyName>
    <OutputPath Condition="'$(ArtifactsPath)' == ''">$(EngineDir)\Binaries\DotNET\AutomationTool\AutomationScripts\AutomatedPerfTesting</OutputPath>
    <TreatWarningsAsErrors>true</TreatWarningsAsErrors>
    <WarningsNotAsErrors>612,618</WarningsNotAsErrors>
  </PropertyGroup>
  <ItemGroup>
    <ProjectReference Include="$(EngineDir)\Source\Programs\AutomationTool\AutomationUtils\AutomationUtils.Automation.csproj" />
    <ProjectReference Include="$(EngineDir)\Source\Programs\Shared\EpicGames.Core\EpicGames.Core.csproj" PrivateAssets="All">
      <Private>false</Private>
    </ProjectReference>
    <ProjectReference Include="$(EngineDir)\Source\Programs\AutomationTool\Gauntlet\Gauntlet.Automation.csproj" PrivateAssets="All">
      <Private>false</Private>
    </ProjectReference>
    <ProjectReference Include="$(EngineDir)\Source\Programs\UnrealBuildTool\UnrealBuildTool.csproj" />
  </ItemGroup>
</Project>
