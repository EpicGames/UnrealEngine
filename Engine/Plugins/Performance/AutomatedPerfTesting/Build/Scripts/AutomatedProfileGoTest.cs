// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.IO;
using System.Linq;
using Gauntlet;
using AutomationTool;
using Log = Gauntlet.Log;
using UnrealBuildTool;

namespace AutomatedPerfTest
{
	/// <summary>
	/// ProfileGo Test config
	/// </summary>
	public class ProfileGoConfig
	{
		[AutoParamWithNames("", "ProfileGoCmd")]
		public string ProfileGoCommand;

		[AutoParamWithNames("", "ProfileGo.Config")]
		public string ConfigFile;

		[AutoParamWithNames("Default", "ProfileGo")]
		public string Scenario;

		[AutoParamWithNames("", "ProfileGo.Precmds")]
		public string PreCmds;

		[AutoParamWithNames("", "ProfileGo.Run")]
		public string RunCommands;

		[AutoParamWithNames("", "ProfileGo.RunFirst")]
		public string RunFirstCommands;

		[AutoParamWithNames("", "ProfileGo.RunLast")]
		public string RunLastCommands;

		[AutoParamWithNames("", "ProfileGo.ExtraArgs")]
		public string ExtraArgs;

		[AutoParamWithNames(false, "ProfileGo.Uncapped")]
		public bool bUncapped;

		[AutoParamWithNames(true, "ProfileGo.Exit")]
		public bool bExit;

		[AutoParamWithNames(-1.0f, "ProfileGo.Settle")]
		public float Settle;

		[AutoParamWithNames(false, "ProfileGo.SkipSettle")]
		public bool bSkipSettle;

		[AutoParamWithNames(false, "ProfileGo.RetraceZ")]
		public bool bRetraceZ;

		[AutoParamWithNames(false, "ProfileGo.DisablePlayerInput")]
		public bool bDisablePlayerInput;
	}

	/// <summary>
	/// ProfileGo Test config
	/// </summary>
	public class AutomatedProfileGoTestConfig : AutomatedPerfTestConfigBase
	{
		public ProfileGoConfig ProfileGoConfig;
	}

	public abstract class AutomatedProfileGoTestNode<TConfigClass> : AutomatedPerfTestNode<TConfigClass>, IAutomatedPerfTest
		where TConfigClass : AutomatedProfileGoTestConfig, new()
	{
		public AutomatedProfileGoTestNode(UnrealTestContext InContext) : base(InContext)
		{
			ProfileGoConfig = new ProfileGoConfig();
			AutoParam.ApplyParamsAndDefaults(ProfileGoConfig, InContext.TestParams.AllArguments);
		}

		public List<string> GetTestsFromConfig()
		{
			List<string> List = new List<string>();
			return List;
		}

		public override TConfigClass GetConfiguration()
		{
			TConfigClass Config = base.GetConfiguration();
			string DefaultConfigPath = Path.Combine(Context.BuildInfo.ProjectPath.Directory.FullName, "Build", "ProfileGo", "ProfileGo.json");
			UnrealTargetPlatform Platform = Context.GetRoleContext(UnrealTargetRole.Client).Platform;
			Config.DataSourceName = Config.GetDataSourceName(Context.BuildInfo.ProjectName, "ProfileGo");
			if (Config.GetRequiredRoles(UnrealTargetRole.Client).Any())
			{
				foreach (UnrealTestRole ClientRole in Config.GetRequiredRoles(UnrealTargetRole.Client))
				{
					ClientRole.Controllers.Add("AutomatedProfileGoTest");

					// Copy json config file if available.
					string ConfigFile = string.IsNullOrEmpty(ProfileGoConfig.ConfigFile) || !File.Exists(ProfileGoConfig.ConfigFile) ? DefaultConfigPath : ProfileGoConfig.ConfigFile;
					if (File.Exists(ConfigFile))
					{
						if (PlatformTargetSupport.IsHostMountingSupported(Platform))
						{
							// If current device supports host mounting of files, update the path and use host mounting
							ProfileGoConfig.ConfigFile = PlatformTargetSupport.GetHostMountedPath(Platform, ConfigFile);
							Log.Info($"Host Mountable Platform Detected. Updated ProfileGo Config File Path to: {ConfigFile} ");
						}
						else
						{
							Log.Info($"Copying Config File to Device: {ConfigFile}");
							UnrealFileToCopy FileToCopy = new UnrealFileToCopy(ConfigFile, EIntendedBaseCopyDirectory.Profiling, Path.GetFileName(ConfigFile));
							ClientRole.FilesToCopy.Add(FileToCopy);
						}
					}
				}
			}

			ApplyProfileGoParams(Config);
			Config.ProfileGoConfig = ProfileGoConfig;
			return Config;
		}

		protected override string GetNormalizedInsightsFileName(string CSVFileName)
		{
			return GetNormalizedInsightsFileName(CSVFileName, "_ProfileGo");
		}

		protected virtual void ApplyProfileGoParams(TConfigClass Config)
		{
			// FYI: ProfileGo in APT is WIP and is subject to change.
			// Applying default values if none available.
			if (string.IsNullOrEmpty(ProfileGoConfig.Scenario))
			{
				ProfileGoConfig.Scenario = "Default";
			}

			if (string.IsNullOrEmpty(ProfileGoConfig.RunCommands))
			{
				ProfileGoConfig.RunCommands = "profileperfspinning,screenshotnamed,meshstats";
			}

			foreach (UnrealTestRole ClientRole in Config.RequiredRoles[UnrealTargetRole.Client])
			{
				if (string.IsNullOrEmpty(ProfileGoConfig.ConfigFile) == false) { ClientRole.CommandLineParams.Add("profilego.config", ProfileGoConfig.ConfigFile); }
				if (string.IsNullOrEmpty(ProfileGoConfig.ProfileGoCommand) == false) { ClientRole.CommandLineParams.Add("profilegocmd", ProfileGoConfig.ProfileGoCommand); }
				if (string.IsNullOrEmpty(ProfileGoConfig.Scenario) == false) { ClientRole.CommandLineParams.Add("profilego", ProfileGoConfig.Scenario); }
				if (string.IsNullOrEmpty(ProfileGoConfig.PreCmds) == false) { ClientRole.CommandLineParams.Add("profilego.precmds", ProfileGoConfig.PreCmds); }
				if (string.IsNullOrEmpty(ProfileGoConfig.RunCommands) == false) { ClientRole.CommandLineParams.Add("profilego.run", ProfileGoConfig.RunCommands); }
				if (string.IsNullOrEmpty(ProfileGoConfig.RunFirstCommands) == false) { ClientRole.CommandLineParams.Add("profilego.runfirst", ProfileGoConfig.RunFirstCommands); }
				if (string.IsNullOrEmpty(ProfileGoConfig.RunLastCommands) == false) { ClientRole.CommandLineParams.Add("profilego.runlast", ProfileGoConfig.RunLastCommands); }
				if (string.IsNullOrEmpty(ProfileGoConfig.ExtraArgs) == false) { ClientRole.CommandLineParams.Add("profilego.extraargs", ProfileGoConfig.ExtraArgs); }
				if (ProfileGoConfig.bUncapped) { ClientRole.CommandLineParams.Add("profilego.uncapped"); }
				if (ProfileGoConfig.bExit) { ClientRole.CommandLineParams.Add("profilego.exit"); }
				if (ProfileGoConfig.Settle > 0.0f) { ClientRole.CommandLineParams.Add("profilego.settle", ProfileGoConfig.Settle); }
				if (ProfileGoConfig.bDisablePlayerInput) { ClientRole.CommandLineParams.Add("profilego.disableplayerinput"); }
			}
		}

		protected override bool CopyCSVs(List<FileInfo> CsvFiles, DirectoryInfo CSVDirectory)
		{
			// For ProfileGo, each "scenario" can generate a CSV file, so we have
			// to ensure we copy all valid CSVs to output directory
			if(CsvFiles.Count == 0)
			{
				return false;
			}

			foreach (FileInfo CsvFile in CsvFiles)
			{
				CopyCSV(CsvFile, CSVDirectory);
			}

			return true;
		}

		private ProfileGoConfig ProfileGoConfig;
	}

	public class ProfileGoTest : AutomatedProfileGoTestNode<AutomatedProfileGoTestConfig>
	{
		public ProfileGoTest(UnrealTestContext InContext)
			: base(InContext) { }
	}
}
