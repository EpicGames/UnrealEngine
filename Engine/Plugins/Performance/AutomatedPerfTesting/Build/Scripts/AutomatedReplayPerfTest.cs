// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using Gauntlet;
using AutomationTool;
using Log = Gauntlet.Log;
using UnrealBuildTool;

namespace AutomatedPerfTest
{
	public class AutomatedReplayPerfTestConfig : AutomatedPerfTestConfigBase
	{
		/// <summary>
		/// Which replay to run the test on
		/// </summary>
		[AutoParamWithNames("", "AutomatedPerfTest.ReplayPerfTest.ReplayName")]
		public string ReplayName;
	}

	/// <summary>
	/// Implementation of a Gauntlet TestNode for AutomatedPerfTest plugin
	/// </summary>
	/// <typeparam name="TConfigClass"></typeparam>
	public abstract class AutomatedReplayPerfTestNode<TConfigClass> : AutomatedPerfTestNode<TConfigClass>, IAutomatedPerfTest
		where TConfigClass : AutomatedReplayPerfTestConfig, new()
	{
		public AutomatedReplayPerfTestNode(UnrealTestContext InContext) : base(InContext)
		{
			Config = null;
		}

		/// <summary>
		/// Handler to explicitly copy files to the device. It is assumed this
		/// is called when Gauntlet configures the device before starting the
		/// tests.
		/// </summary>
		public void CopyReplayToDevice(ITargetDevice Device)
		{
			if (Config == null)
			{
				Log.Warning("ConfigureDevice() called before Test Node Configuration is set.");
				return;
			}

			// At the moment in some pathways we don't seem to copy
			// files in a role. This ensures we have a copy of the
			// replay files on the target device regardless.
			if (Config.GetRequiredRoles(UnrealTargetRole.Client).Any())
			{
				foreach (UnrealTestRole ClientRole in Config.GetRequiredRoles(UnrealTargetRole.Client))
				{
					UnrealTestRoleContext RoleContext = Context.GetRoleContext(UnrealTargetRole.Client);
					UnrealTargetPlatform Platform = RoleContext.Platform;
					if (Device.Platform == Platform && ClientRole.FilesToCopy.Any())
					{
						UnrealSessionRole Role = new(RoleContext.Type, Platform, RoleContext.Configuration);
						UnrealAppConfig AppConfig = Context.BuildInfo.CreateConfiguration(Role);

						// needed so AFS will work on CopyAdditional files
						_ = Device.CreateAppInstall(AppConfig);

						Device.CopyAdditionalFiles(ClientRole.FilesToCopy);
					}
				}
			}
		}

		public override TConfigClass GetConfiguration()
		{
			Config = base.GetConfiguration();

			Config.DataSourceName = Config.GetDataSourceName(Context.BuildInfo.ProjectName, ReplayDataSourceName);

			IEnumerable<UnrealTestRole> ClientRoles = Config.GetRequiredRoles(UnrealTargetRole.Client);
			// extend the role(s) that we initialized in the base class
			if (ClientRoles.Any())
			{
				foreach (UnrealTestRole ClientRole in ClientRoles)
				{
					ClientRole.Controllers.Add("AutomatedReplayPerfTest");
					ClientRole.FilesToCopy.Clear();

					UnrealTargetPlatform Platform = Context.GetRoleContext(UnrealTargetRole.Client).Platform;
					List<string> ReplaysFromConfig = GetTestsFromConfig();

					// Replay name not provided
					if (string.IsNullOrEmpty(Config.ReplayName))
					{
						if(ReplaysFromConfig == null || !ReplaysFromConfig.Any())
						{
							throw new AutomationException("No replays found in settings or provided via arguments.");
						}

						// Use the first replay found. We need to extract the replay path from settings as we need to
						// post process the path depending on the target platform.
						Config.ReplayName = ReplaysFromConfig[0];
					}
					else
					{
						foreach(string Replay in ReplaysFromConfig)
						{
							if(!string.IsNullOrEmpty(Replay) && Replay.Contains(Config.ReplayName))
							{
								Log.Info("Found replay in settings");
								Config.ReplayName = Replay;
								break;
							}
						}
					}

					string ReplayName = Path.GetFullPath(Config.ReplayName);
					bool bFileExists = File.Exists(ReplayName);
					if (!bFileExists)
					{
						// In case the replay file specified is not in settings
						// or not an absolute path. Check under project directory
						// to be sure.
						ReplayName = GetPathInProject(Context, Config.ReplayName);
						bFileExists = File.Exists(ReplayName);
					}

					if(!bFileExists)
					{
						throw new AutomationException($"Replay file '{Config.ReplayName}' not found");
					}

					if (PlatformTargetSupport.IsHostMountingSupported(Platform))
					{
						// If current device supports host mounting of files, update the path and use host mounting
						ReplayName = PlatformTargetSupport.GetHostMountedPath(Platform, ReplayName);
						Log.Info($"Host Mountable Platform Detected. Updated Replay File Path to: {ReplayName} ");
					}
					else
					{
						// Copy replay file to Demos folder in device if host mounting is not supported
						Log.Info($"Copy Replay File Path to Device: {ReplayName}");
						UnrealFileToCopy FileToCopy = new UnrealFileToCopy(ReplayName, EIntendedBaseCopyDirectory.Demos, Path.GetFileName(ReplayName));
						ClientRole.FilesToCopy.Add(FileToCopy);
						ClientRole.ConfigureDevice = CopyReplayToDevice;

						// If we copy to the "Demos" folder, we just need to pass the Replay Name without
						// path and extension as the replay subsystem will automatically pick up the file
						// from this folder.
						ReplayName = Path.GetFileNameWithoutExtension(ReplayName);
					}

					ClientRole.CommandLineParams.AddUnique(ReplayParamName, ReplayName);
					Log.Info($"{ReplayParamName}=\"{ReplayName}\"");
				}
			}

			return Config;
		}

		public List<string> GetTestsFromConfig()
		{
			List<string> OutReplayList = new List<string>();
			ReadConfigArray(Context,
				"/Script/AutomatedPerfTesting.AutomatedReplayPerfTestProjectSettings",
				"ReplaysToTest",
				Config =>
				{
					Dictionary<string, string> ReplayConfig = IniConfigUtil.ParseDictionaryFromConfigString(Config);
					string Path;
					if (ReplayConfig.TryGetValue("FilePath", out Path))
					{
						OutReplayList.Add(GetPathInProject(Context, Path.Replace("\"", "")));
					}
				});

			return OutReplayList;
		}

		protected override string GetNormalizedInsightsFileName(string CSVFileName)
		{
			return GetNormalizedInsightsFileName(CSVFileName, "_Replay");
		}

		/// <inheritdoc/>
		protected override string CreateTestIdentity(TConfigClass Config = null)
		{
			Config ??= GetCachedConfiguration();

			string rootTestIdentity = base.CreateTestIdentity(Config);
			string replayName = string.IsNullOrEmpty(Config.ReplayName) ? string.Empty : Path.GetFileNameWithoutExtension(Config.ReplayName);

			return string.IsNullOrEmpty(replayName) ? rootTestIdentity : $"{rootTestIdentity}.{replayName}";
		}

		private readonly string ReplayDataSourceName = "ReplayRun";
		private readonly string ReplayParamName = "AutomatedPerfTest.ReplayPerfTest.ReplayName";
		private TConfigClass Config;
	}

	/// <summary>
	/// "Standard issue" implementation usable for samples that don't need anything more advanced
	/// </summary>
	public class ReplayTest : AutomatedReplayPerfTestNode<AutomatedReplayPerfTestConfig>
	{
		public ReplayTest(Gauntlet.UnrealTestContext InContext)
			: base(InContext)
		{
		}
	}
}
