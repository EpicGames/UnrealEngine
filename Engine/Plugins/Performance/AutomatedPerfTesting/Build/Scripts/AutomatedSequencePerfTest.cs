// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using Gauntlet;
using AutomationTool;

namespace AutomatedPerfTest
{
	public class AutomatedSequencePerfTestConfig : AutomatedPerfTestConfigBase
	{
		/// <summary>
		/// Which map to run the test on
		/// </summary>
		[AutoParamWithNames("", "AutomatedPerfTest.SequencePerfTest.MapSequenceName")]
		public string MapSequenceComboName;
	}

	/// <summary>
	/// Implementation of a Gauntlet TestNode for AutomatedPerfTest plugin
	/// </summary>
	/// <typeparam name="TConfigClass"></typeparam>
	public abstract class AutomatedSequencePerfTestNode<TConfigClass> : AutomatedPerfTestNode<TConfigClass>, IAutomatedPerfTest
		where TConfigClass : AutomatedSequencePerfTestConfig, new()
	{
		public AutomatedSequencePerfTestNode(UnrealTestContext InContext) : base(InContext)
		{
			SummaryTable = "sequence";
		}

		public override TConfigClass GetConfiguration()
		{
			TConfigClass Config = base.GetConfiguration();

			Config.DataSourceName = Config.GetDataSourceName(Context.BuildInfo.ProjectName, "Sequence");

			// extend the role(s) that we initialized in the base class
			if (Config.GetRequiredRoles(UnrealTargetRole.Client).Any())
			{
				foreach(UnrealTestRole ClientRole in Config.GetRequiredRoles(UnrealTargetRole.Client))
				{
					ClientRole.Controllers.Add("AutomatedSequencePerfTest");

					// if a specific MapSequenceComboName was defined in the commandline to UAT, then add that to the commandline for the role
					if (!string.IsNullOrEmpty(Config.MapSequenceComboName))
					{
						// use add Unique, since there should only ever be one of these specified
						ClientRole.CommandLineParams.AddUnique($"AutomatedPerfTest.SequencePerfTest.MapSequenceName",
							Config.MapSequenceComboName);
					}
				}
			}

			return Config;
		}

		public List<string> GetTestsFromConfig()
		{
			List<string> OutSequenceList = new List<string>();
			ReadConfigArray(Context,
				"/Script/AutomatedPerfTesting.AutomatedSequencePerfTestProjectSettings",
				"MapsAndSequencesToTest",
				Config =>
				{
					Dictionary<string, string> SequenceConfig = IniConfigUtil.ParseDictionaryFromConfigString(Config);
					string ComboName;
					if (SequenceConfig.TryGetValue("ComboName", out ComboName))
					{
						OutSequenceList.Add(ComboName.Replace("\"", ""));
					}
				});

			return OutSequenceList;
		}

		protected override string GetNormalizedInsightsFileName(string CSVFileName)
		{
			return GetNormalizedInsightsFileName(CSVFileName, "_Sequence");
		}

		/// <inheritdoc/>
		protected override string CreateTestIdentity(TConfigClass Config = null)
		{
			Config ??= GetCachedConfiguration();

			string rootTestIdentity = base.CreateTestIdentity(Config);

			return $"{rootTestIdentity}.{Config.MapSequenceComboName}";
		}
	}

	/// <summary>
	/// "Standard issue" implementation usable for samples that don't need anything more advanced
	/// </summary>
	public class SequenceTest : AutomatedSequencePerfTestNode<AutomatedSequencePerfTestConfig>
	{
		public SequenceTest(Gauntlet.UnrealTestContext InContext)
			: base(InContext)
		{
		}
	}
}
