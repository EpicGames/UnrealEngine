// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Linq;
using Gauntlet;

namespace AutomatedPerfTest
{
	public class AutomatedStaticCameraPerfTestConfig : AutomatedPerfTestConfigBase
	{
		/// <summary>
		/// Which map to run the test on
		/// </summary>
		[AutoParamWithNames("", "AutomatedPerfTest.StaticCameraPerfTest.MapName")]
		public string MapName;
	}

	/// <summary>
	/// Implementation of a Gauntlet TestNode for AutomatedPerfTest plugin
	/// </summary>
	/// <typeparam name="TConfigClass"></typeparam>
	public abstract class AutomatedStaticCameraPerfTestNode<TConfigClass> : AutomatedPerfTestNode<TConfigClass>, IAutomatedPerfTest
		where TConfigClass : AutomatedStaticCameraPerfTestConfig, new()
	{
		public AutomatedStaticCameraPerfTestNode(UnrealTestContext InContext) : base(InContext)
		{
			SummaryTable = "staticcamera";
		}

		public List<string> GetTestsFromConfig()
		{
			List<string> OutMaps = new List<string>();
			ReadConfigArray(Context,
				"/Script/AutomatedPerfTesting.AutomatedStaticCameraPerfTestProjectSettings",
				"MapsToTest",
				Map => OutMaps.Add(Map.Replace("\"", "")));

			return OutMaps;
		}

		public override TConfigClass GetConfiguration()
		{
			TConfigClass Config = base.GetConfiguration();

			Config.DataSourceName = Config.GetDataSourceName(Context.BuildInfo.ProjectName, "StaticCamera");

			// extend the role(s) that we initialized in the base class
			if (Config.GetRequiredRoles(UnrealTargetRole.Client).Any())
			{
				foreach(UnrealTestRole ClientRole in Config.GetRequiredRoles(UnrealTargetRole.Client))
				{
					ClientRole.Controllers.Add("AutomatedPlacedStaticCameraPerfTest");

					// if a specific MapName was defined in the commandline to UAT, then add that to the commandline for the role
					if (!string.IsNullOrEmpty(Config.MapName))
					{
						// use add Unique, since there should only ever be one of these specified
						ClientRole.CommandLineParams.AddUnique($"AutomatedPerfTest.StaticCameraPerfTest.MapName",
							Config.MapName);
					}
				}
			}

			return Config;
		}

		protected override string GetNormalizedInsightsFileName(string CSVFileName)
		{
			return GetNormalizedInsightsFileName(CSVFileName, "_StaticCamera");
		}
	}

	/// <summary>
	/// "Standard issue" implementation usable for samples that don't need anything more advanced
	/// </summary>
	public class StaticCameraTest : AutomatedStaticCameraPerfTestNode<AutomatedStaticCameraPerfTestConfig>
	{
		public StaticCameraTest(Gauntlet.UnrealTestContext InContext)
			: base(InContext)
		{
		}
	}
}
