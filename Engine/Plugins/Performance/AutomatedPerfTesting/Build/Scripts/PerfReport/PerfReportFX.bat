@echo off
call %~dp0\SetupEnv.bat

if [%~1]==[] (
  %CSVToolsPath%\PerfReportTool -reporttype=FXReport
  goto exit
)

if [%~2]==[] (
  @echo on
  %CSVToolsPath%\PerfReportTool -reporttype FXReport -reportxmlbasedir %~dp0 -csv %1
  goto exit
) 

%CSVToolsPath%\PerfReportTool -reporttype FXReport -reportxmlbasedir %~dp0 %* 

:exit
