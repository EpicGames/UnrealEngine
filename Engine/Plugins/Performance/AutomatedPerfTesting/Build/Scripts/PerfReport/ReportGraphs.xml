<?xml version="1.0" encoding="UTF-8"?>

<graphGroups>
	<baseSettings thickness="2" smooth="1" smoothKernelSize="60" compression="0.5"
				  width="1920" height="1080" stacked="0" maxHierarchyDepth="2" hideStatPrefix="" showEvents="App_* HitchDetector GC Sandbox* OT* AutomatedPerfTest*" showAverages="1"/>

	<graphGroup name="Core">
		<baseSettings smoothKernelSize="60" smoothKernelPercent="-1"/>

		<graph title="Stat Unit" showAverages="1" thickness="3">
			<statString>frametime,gamethreadtime,renderthreadtime,gputime,rhithreadtime,MaxFrameTime</statString>
		</graph>

		<graph title="Stat Unit Raw" showAverages="1" thickness="1" smooth="0" maxy="0">
			<statString>frametime,gamethreadtime,renderthreadtime,gputime,rhithreadtime,MaxFrameTime</statString>
		</graph>

		<graph title="Stat Unit 30hz" showAverages="1" maxy="60" thickness="3">
			<statString>frametime,gamethreadtime,renderthreadtime,gputime,rhithreadtime,MaxFrameTime</statString>
		</graph>

		<graph title="Stat Unit 20hz" showAverages="1" maxy="80" thickness="3">
			<statString>frametime,gamethreadtime,renderthreadtime,gputime,rhithreadtime,MaxFrameTime</statString>
		</graph>

		<graph title="Dynamic Resolution" maxy="100" thickness="4" showAverages="1">
			<statString>DynamicResolutionPercentage,MaxFrameTime</statString>
		</graph>

		<graph title="Cmds DataLayers"  smooth="0" thickness="4" miny="5000" maxy="25000" showEvents="Cmd* DataLayer*" >
			<statString>PhysicalUsedMB, SystemMaxMB, MemoryFreeMB</statString>
		</graph>
		<graph title="Physical Memory"  smooth="0" thickness="4" maxy="0" >
			<statString>PhysicalUsedMB,MemoryFreeMB,SystemMaxMB</statString>
		</graph>
		
		<graph title="Physical Memory Over Budget"  smooth="0" thickness="4" maxy="3000">
			<statString>PhysicalUsedMB, MemoryFreeMB,ExtendedUsedMB</statString>
		</graph>

		<graph title="RHI Drawcalls" maxy="4000" thickness="4">
			<statString>RHI/Drawcalls</statString>
		</graph>

		<graph title="RHI Drawcalls breakdown" hideStatPrefix="DrawCall/" stacked="1" thickness="4" smooth="0" snapToPeaks="0" maxY="0" showAverages="0" stackTotalStat="RHI/Drawcalls">
			<statString>Drawcall/*</statString>
		</graph>

		<graph title="Gamethread Breakdown" showAverages="1" hideStatPrefix="exclusive/Gamethread/" stacked="1" mainStat="gamethreadtime" ignoreStats="exclusive/Gamethread/EventWait*" legendAverageThreshold="0.1">
			<statString>exclusive/gamethread/*,gamethreadtime</statString>
		</graph>

		<graph title="Gamethread Breakdown Raw" hideStatPrefix="exclusive/Gamethread/" stacked="1" mainStat="gamethreadtime" ignoreStats="exclusive/Gamethread/EventWait*" smooth="0"  maxy="100" legendAverageThreshold="0.1">
			<statString>exclusive/gamethread/*,gamethreadtime</statString>
		</graph>

		<graph title="Gamethread Waits" showAverages="1" hideStatPrefix="exclusive/gamethread/eventwait/" stacked="1" mainStat="frametime" ignoreStats="exclusive/Gamethread/EventWait" maxHierarchyDepth="3">
			<statString>exclusive/gamethread/eventwait/*,gamethreadtime</statString>
		</graph>

		<graph title="Gamethread Physics" showAverages="1" stacked="1" maxy="0" maxHierarchyDepth="3" hideStatPrefix="exclusive/gamethread/">
		  <statString>exclusive/gamethread/eventwait/EndPhysics,exclusive/gamethread/Physics,exclusive/gamethread/vehicle*</statString>
		</graph>

		<graph title="Gamethread Breakdown Giant" hideStatPrefix="exclusive/Gamethread/" stacked="1" mainStat="gamethreadtime" ignoreStats="exclusive/Gamethread/EventWait*" smooth="0"  maxy="100" legendAverageThreshold="0.1" width="40000" height="1000">
			<statString>exclusive/gamethread/*,gamethreadtime</statString>
		</graph>

		<graph title="Gamethread Budget: Effects" showAverages="1" stacked="1" maxy="0" hideStatPrefix="exclusive/gamethread/">
		  <statString>exclusive/gamethread/effects</statString>
		</graph>
    
		<graph title="Renderthread Breakdown" showAverages="1" hideStatPrefix="exclusive/RenderThread/" stacked="1" mainStat="renderthreadtime" ignoreStats="exclusive/Renderthread/FetchVisibilityForPrimitives;exclusive/renderthread/eventwait*">
			<statString>exclusive/renderthread/*,renderthreadtime</statString>
		</graph>

		<graph title="Renderthread Breakdown Raw" hideStatPrefix="exclusive/RenderThread/" stacked="1" mainStat="renderthreadtime" ignoreStats="exclusive/Renderthread/FetchVisibilityForPrimitives;exclusive/renderthread/eventwait*" smooth="0"  maxy="100">
			<statString>exclusive/renderthread/*,renderthreadtime</statString>
		</graph>

		<graph title="Render Targets Peak(MB)" showAverages="0" stacked="1" mainStat="RenderTargetProfiler/Total" smooth="0" maxy="0" hideStatPrefix="RenderTargetProfiler/">
			<statString>RenderTargetProfiler/*</statString>
		</graph>
		
		<graph title="Textures Peak(MB)" showAverages="0" stacked="1" mainStat="TextureProfiler/Total" smooth="0" maxy="0" hideStatPrefix="TextureProfiler/">
			<statString>TextureProfiler/*</statString>
		</graph>

		<graph title="Virtual Texture" showAverages="0" maxy="0" hideStatPrefix="RenderTargetPool/" stacked="1">
			<statString>VirtualTexturePageUsage, VirtualUsedMB</statString>
		</graph>

		<graph title="GPU Breakdown" showAverages="1" smoothKernelSize="120" hideStatPrefix="gpu/" stacked="1" mainStat="gpu/total" maxy="80">
			<statString>GPU/*</statString>
		</graph>

		<graph title="GPU Breakdown Raw" showAverages="1" smooth="0" hideStatPrefix="gpu/" stacked="1" mainStat="gpu/total" maxy="80">
			<statString>GPU/*</statString>
		</graph>

		<graph title="D3D12 Video Memory" showAverages="1" stacked="1" maxy="0">
			<statString>D3D12Memory*</statString>
		</graph>

		<graph title="Niagara Breakdown" thickness="2" ignoreStats="global*" smooth="1" smoothKernelSize="10" maxy="60">
			<statString>Ticks/NiagaraComponent,Exclusive/GameThread/Niagara,GPU/NiagaraGPUSimulation,GPU/NiagaraGPUSorting</statString>
		</graph>

		<graph title="Light Counts" hideStatPrefix="LightCount/" thickness="4" smooth="0" snapToPeaks="0" maxy="0" showAverages="0">
			<statString>LightCount/*</statString>
		</graph>

		<graph title="Ticks Queued" showAverages="0" maxy="0" thickness="4" smooth="0">
			<statString>Basic/TicksQueued</statString>
		</graph>

		<graph title="Ticks Breakdown" showAverages="1" maxy="0" maxHierarchyDepth="4" hideStatPrefix="Ticks/" stacked="1" mainStat="Basic/TicksQueued" threshold="1">
			<statString>Ticks/*</statString>
		</graph>

		<graph title="Actor Counts" showAverages="0" maxy="0" thickness="4" hideStatPrefix="ActorCount/" smooth="0" stacked="1" mainStat="ActorCount/TotalActorCount">
			<statString>ActorCount/*</statString>
		</graph>

		<graph title="Animation Budget" showAverages="0" hideStatPrefix="Exclusive/GameThread/" mainStat="Exclusive/GameThread/Animation" thickness="4" smooth="1" filterOutZeros="0">
			<statString>Exclusive/GameThread/Animation,AnimationBudget/*</statString>
		</graph>

		<graph title="Log Counts" showAverages="1" hideStatPrefix="fmsglogf/" mainStat="fmsglogf/fmsglogfcount" thickness="4" smooth="0" filterOutZeros="0">
			<statString>fmsglogf/*</statString>
		</graph>

		<graph title="File I/O" showAverages="1" hideStatPrefix="FileIO/" thickness="8" smooth="0" maxY="0" showEvents="LevelStreaming/*">
			<statString>FileIO/EDLEventQueueDepth,FileIO/ExistingQueuedPackagesQueueDepth,FileIO/QueuedPackagesQueueDepth,IoDispatcher/PendingIoRequests,FileIO/PendingAsyncLoadingIoRequestsSizeMB,IoDispatcherFileBackend/QueuedFilesystemReadMB</statString>
		</graph>

		<graph title="Texture Streaming" showAverages="1" hideStatPrefix="texturestreaming/" mainStat="textureStreaming/streamingPool" thickness="4" smooth="0" snapToPeaks="0" maxy="0">
			<statString>texturestreaming/*</statString>
		</graph>

		<graph title="FX Breakdown Total" showAverages="1" stacked="1" maxy="0" fixedlegend="1" dynamicranges="1" hideStatPrefix="Particles/Total/" showEvents="Particles*">
		  <statString>Particles/Total/*</statString>
		</graph>
		
		<graph title="FX Breakdown GTOnly" showAverages="1" stacked="1" maxy="0" fixedlegend="1" dynamicranges="1" hideStatPrefix="Particles/GTOnly/" showEvents="Particles*">
		  <statString>Particles/GTOnly/*</statString>
		</graph>
		
		<graph title="FX Breakdown RT" showAverages="1" stacked="1" maxy="0" fixedlegend="1" dynamicranges="1" hideStatPrefix="Particles/RT/" showEvents="Particles*">
		  <statString>Particles/RT/*</statString>
		</graph>
		
		<graph title="FX Breakdown GPU" showAverages="1" stacked="1" maxy="0" fixedlegend="1" dynamicranges="1" hideStatPrefix="Particles/GPU/" showEvents="Particles*">
		  <statString>Particles/GPU/*</statString>
		</graph>
		
		<graph title="FX Breakdown Per Inst Avg GT" showAverages="1" stacked="0" maxy="0" fixedlegend="1" dynamicranges="1" hideStatPrefix="Particles/InstAvgGT/" showEvents="Particles*">
		  <statString>Particles/InstAvgGT/*</statString>
		</graph>
		
		<graph title="FX Breakdown Per Inst Avg RT" showAverages="1" stacked="0" maxy="0" fixedlegend="1" dynamicranges="1" hideStatPrefix="Particles/InstAvgRT/" showEvents="Particles*">
		  <statString>Particles/InstAvgRT/*</statString>
		</graph>
		
		<graph title="FX Breakdown Per Inst Avg GPU" showAverages="1" stacked="0" maxy="0" fixedlegend="1" dynamicranges="1" hideStatPrefix="Particles/InstAvgGPU/" showEvents="Particles*">
		  <statString>Particles/InstAvgGPU/*</statString>
		</graph>		
		
		<graph title="FX Breakdown Count" showAverages="0" stacked="1" maxy="0" fixedlegend="1" dynamicranges="1" hideStatPrefix="Particles/Count/" showEvents="Particles*">
		  <statString>Particles/Count/*</statString>
		</graph>
		
		<graph title="FX Breakdown Activation" showAverages="0" stacked="1" maxy="0" fixedlegend="1" dynamicranges="1" hideStatPrefix="Particles/Activation/" showEvents="Particles*">
		  <statString>Particles/Activation/*</statString>
		</graph>
		
		<graph title="FX Breakdown Waits" showAverages="0" stacked="1" maxy="0" fixedlegend="1" dynamicranges="1" hideStatPrefix="Particles/Waits/" showEvents="Particles*">
		  <statString>Particles/Waits/*</statString>
		</graph>
		
		<graph title="Niagara Culling" showAverages="0" stacked="0" maxy="0" fixedlegend="1" dynamicranges="1" hideStatPrefix="Particles/NiagaraCulled/" showEvents="Particles*">
		  <statString>Particles/NiagaraCulled/*,Particles/TotalCullProxies</statString>
		</graph>
		
		<graph title="Niagara Culling Breakdown" showAverages="0" stacked="1" maxy="0" fixedlegend="1" dynamicranges="1" hideStatPrefix="Particles/Culled/" showEvents="Particles*">
		  <statString>Particles/Culled/*</statString>
		</graph>
		
		<graph title="Niagara Cull Proxy Breakdown" showAverages="0" stacked="1" maxy="0" fixedlegend="1" dynamicranges="1" hideStatPrefix="Particles/NumCullProxies/" showEvents="Particles*">
		  <statString>Particles/NumCullProxies/*</statString>
		</graph>
		
		<graph title="FX Budget Breakdown" showAverages="1" stacked="0" maxy="0" fixedlegend="1" dynamicranges="1" hideStatPrefix="Particles/Budget/" showEvents="Particles*">
		  <statString>Particles/Budget/*</statString>
		</graph>

		<graph title="CSV Profiler" showAverages="1" hideStatPrefix="CSVProfiler/" stacked="1" mainStat="frametime" maxy="0">
			<statString>CSVProfiler/Num*</statString>
		</graph>

		<graph title="FScene Breakdown" showAverages="1" hideStatPrefix="exclusive/RenderThread/" stacked="1" mainStat="Exclusive/RenderThread/AddPrimitiveSceneInfos">
			<statString>Exclusive/RenderThread/UpdatePrimitiveTransform,Exclusive/RenderThread/UpdatePrimitiveInstances,Exclusive/RenderThread/AddPrimitiveSceneInfos,Exclusive/RenderThread/RemovePrimitiveSceneInfos</statString>
		</graph>

		<graph title="FScene Breakdown Raw" hideStatPrefix="exclusive/RenderThread/" stacked="1" mainStat="Exclusive/RenderThread/AddPrimitiveSceneInfos" smooth="0"  maxy="100">
			<statString>Exclusive/RenderThread/UpdatePrimitiveTransform,Exclusive/RenderThread/UpdatePrimitiveInstances,Exclusive/RenderThread/AddPrimitiveSceneInfos,Exclusive/RenderThread/RemovePrimitiveSceneInfos</statString>
		</graph>
		
		<graph title="Nanite Streaming" hideStatPrefix="NaniteStreaming/" thickness="4" smooth="0" snapToPeaks="1" maxY="0" showAverages="0" showEvents="Nanite*">
			<statString>NaniteStreaming/*</statString>
		</graph>

		<graph title="RenderTarget Pool" showAverages="1" thickness="4" smooth="0" snapToPeaks="1" maxY="0">
			<statString>RenderTargetPool/*,DistanceField/AtlasMB,ShadowCacheUsageMB</statString>
		</graph>

		<graph title="MallocBinned Bins Fragmentation (%)" showAverages="1" maxy="0" hideStatPrefix="MallocBinned/">
			<statString>MallocBinned/FragmentationBin*</statString>
		</graph>
		
		<graph title="MallocBinned Bins Waste (MB)" showAverages="1" maxy="0" stacked="1" hideStatPrefix="MallocBinned/">
			<statString>MallocBinned/WasteBin*</statString>
		</graph>
		
		<graph title="MallocBinned Total Bin Mem (MB)" showAverages="1" maxy="0" stacked="1" hideStatPrefix="MallocBinned/">
			<statString>MallocBinned/TotalMemBin*</statString>
		</graph>
		
		<graph title="MallocBinned Allocations Breakdown" showAverages="1" maxy="0" hideStatPrefix="MallocBinned/">
			<statString>MallocBinned/RequestedSmallPoolMemoryMB,MallocBinned/TotalAllocatedSmallPoolMemoryMB,MallocBinned/RequestedLargeAllocsMemoryMB,MallocBinned/TotalAllocatedLargeAllocsMemoryMB</statString>
		</graph>

		<graph title="Object Count" showAverages="0" maxY="0" thickness="4" hideStatPrefix="ObjectCount/" smooth="0" snapToPeaks="1">
			<statString>ObjectCount/Total</statString>
		</graph>
		<!-- Add more graphs here. -->
	</graphGroup>


	<!-- a graph group is a convenient way to apply common settings to multiple graphs, avoiding duplication -->
	<graphGroup name="LLMBase">
	<baseSettings maxHierarchyDepth="3" stacked="0" hideStatPrefix="LLM/" requiresDetailedStats="0" thickness="10" compression="0.5" showEvents="GC AutomatedPerfTest*"/>

		<!-- ignoreStats should include all parent LLM tags -->
		<graph title="Tracked Memory Breakdown" stacked="1" thickness="1" maxHierarchyDepth="1" ignoreStats="LLM/Total;LLM/Untracked" mainStat="LLM/TrackedTotal">
			<statString>LLM/*</statString>
		</graph>

		<!-- top level LLM summary graphs -->
		<graph title="Total Memory Stats">
			<statString>LLM/Total,LLM/Untracked,LLM/TrackedTotal</statString>
		</graph>
		<graph title="Misc Memory Stats">
			<statString>LLM/EngineInit,LLM/EnginePreInit,LLM/EngineMisc,LLM/TaskGraphMiscTasks,LLM/FName,LLM/GC,LLM/Localization,LLM/AssetRegistry,LLM/ConfigSystem,LLM/RealTimeCommunications,LLM/StreamingManager</statString>
		</graph>
		<graph title="Rendering Memory Stats">
			<statString>LLM/RenderingThread,LLM/RenderTargets,LLM/PSO,LLM/SceneRender,LLM/RHIMisc,LLM/Graphics</statString>
		</graph>
		<graph title="Development Memory Stats">
			<statString>LLM/Stats,LLM/CsvProfiler</statString>
		</graph>
		<graph title="Initialization Memory Stats">
			<statString>LLM/EnginePreInit,LLM/EngineInit,LLM/ProgramSize</statString>
		</graph>
		<graph title="Loading Memory Stats">
			<statString>LLM/AsyncLoading,LLM/FileSystem,LLM/StreamingManager,LLM/LoadMapMisc</statString>
		</graph>


		<!-- Memory breakdowns -->
		<graph title="Content Memory Breakdown" stacked="1" thickness="1" maxHierarchyDepth="2" hideStatPrefix="LLM/Meshes/;LLM/Audio/;LLM/">
			<statString>LLM/Textures,LLM/Shaders,LLM/Materials,LLM/MaterialInstance,LLM/Meshes/*,LLM/Audio/AudioSoundWaves,LLM/Animation</statString>
		</graph>
		<graph title="Audio Memory Breakdown" maxHierarchyDepth="2" hideStatPrefix="LLM/Audio/" stacked="1" stackTotalStat="LLM/Audio">
			<statString>LLM/Audio/*</statString>
		</graph>
		<graph title="Mesh Memory Breakdown" maxHierarchyDepth="2" hideStatPrefix="LLM/Meshes/" stacked="1" stackTotalStat="LLM/Meshes">
			<statString>LLM/Meshes/*</statString>
		</graph>
		<graph title="Physics Memory Breakdown" maxHierarchyDepth="2" hideStatPrefix="LLM/Physics/"  stacked="1" stackTotalStat="LLM/Physics">
			<statString>LLM/Physics/*</statString>
		</graph>


		<!-- Individual stat graphs -->
		<graph title="Tracked Total Memory">
			<statString>LLM/TrackedTotal</statString>
		</graph>
		<graph title="Untagged Memory">
			<statString>LLM/Untagged</statString>
		</graph>
		<graph title="Untracked Memory">
			<statString>LLM/Untracked</statString>
		</graph>
		<graph title="UObject Memory">
			<statString>LLM/UObject</statString>
		</graph>
		<graph title="Shaders Memory">
			<statString>LLM/Shaders</statString>
		</graph>
		<graph title="RenderTargets Memory">
			<statString>LLM/RenderTargets</statString>
		</graph>
		<graph title="Textures Memory">
			<statString>LLM/Textures</statString>
		</graph>
		<graph title="Materials Memory">
			<statString>LLM/Materials</statString>
		</graph>
		<graph title="Animation Memory">
			<statString>LLM/Animation</statString>
		</graph>
		<graph title="EngineMisc Memory">
			<statString>LLM/EngineMisc</statString>
		</graph>
		<graph title="Niagara Memory">
			<statString>LLM/Niagara</statString>
		</graph>
		<graph title="FName Memory">
			<statString>LLM/FName</statString>
		</graph>
		<graph title="GC Memory">
			<statString>LLM/GC</statString>
		</graph>
		<graph title="EnginePreInit Memory">
			<statString>LLM/EnginePreInit</statString>
		</graph>
		<graph title="EngineInit Memory">
			<statString>LLM/EngineInit</statString>
		</graph>
		<graph title="LoadMapMisc Memory">
			<statString>LLM/LoadMapMisc</statString>
		</graph>
		<graph title="LLMPlatform">
			<statString>LLMPlatform/*</statString>
		</graph>
		<graph title="ThreadStack Memory">
			<statString>LLM/ThreadStack</statString>
		</graph>
		<graph title="Networking Memory">
			<statString>LLM/Networking</statString>
		</graph>
		<graph title="MaterialInstance Memory">
			<statString>LLM/MaterialInstance</statString>
		</graph>
		<graph title="TextureMetaData Memory">
			<statString>LLM/TextureMetaData</statString>
		</graph>
		<graph title="SceneRender Memory">
			<statString>LLM/SceneRender</statString>
		</graph>
		<graph title="RHIMisc Memory">
			<statString>LLM/RHIMisc</statString>
		</graph>
		<graph title="InitUObject Memory">
			<statString>LLM/InitUObject</statString>
		</graph>
		<graph title="FMallocUnused Memory">
			<statString>LLM/FMallocUnused</statString>
		</graph>
		<graph title="ConfigSystem Memory">
			<statString>LLM/ConfigSystem</statString>
		</graph>
		<graph title="CSVProfiler Memory">
			<statString>LLM/CSVProfiler</statString>
		</graph>
		<graph title="UI Memory">
			<statString>LLM/UI</statString>
		</graph>
		<graph title="Meshes Memory">
			<statString>LLM/Meshes</statString>
		</graph>
		<graph title="Physics Memory">
			<statString>LLM/Physics</statString>
		</graph>
		<graph title="FileSystem Memory">
			<statString>LLM/Filesystem</statString>
		</graph>
		<graph title="Particles Memory">
			<statString>LLM/Particles</statString>
		</graph>
		<graph title="Effects Memory">
			<statString>LLM/Particles,LLM/Niagara</statString>
		</graph>
		<graph title="Audio Memory">
			<statString>LLM/Audio</statString>
		</graph>
		<graph title="Localization Memory">
			<statString>LLM/Localization</statString>
		</graph>
		<graph title="AssetRegistry Memory">
			<statString>LLM/AssetRegistry</statString>
		</graph>
		<graph title="ProgramSize Memory">
			<statString>LLM/ProgramSize</statString>
		</graph>
		<graph title="AsyncLoading Memory">
			<statString>LLM/AsyncLoading</statString>
		</graph>
		<!--Add more LLM graphs if needed here with respective LLM metric(s)-->
	</graphGroup>

	<graphGroup name="LLMPlatformBase">
		<baseSettings maxHierarchyDepth="4" maxy="1024" stacked="0" hideStatPrefix="" requiresDetailedStats="0" thickness="10" compression="0.5" showEvents="AutomatedPerfTest*" />

		<graph title="Platform FMalloc Memory" maxy="2500">
			<statString>LLMPlatform/FMalloc</statString>
		</graph>
		<graph title="Platform GenericPlatformMallocCrash Memory" maxy="5">
			<statString>LLMPlatform/GenericPlatformMallocCrash</statString>
		</graph>
		<graph title="Platform Graphics Memory" maxy="2000">
			<statString>LLMPlatform/Graphics</statString>
		</graph>
		<graph title="Platform Thread Stack Memory" maxy="10">
			<statString>LLMPlatform/ThreadStack</statString>
		</graph>
	</graphGroup>
</graphGroups>