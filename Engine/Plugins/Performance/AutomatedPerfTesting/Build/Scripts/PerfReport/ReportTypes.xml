<?xml version="1.0" encoding="UTF-8"?>

<root>
	<!-- Friendly names for display purposes -->
	<statDisplayNameMappings>
		<mapping statName="RHI/Drawcalls" displayName="DrawCalls"/>
		<mapping statName="DynamicResolutionPercentage" displayName="DynRes %"/>
		<mapping statName="Basic/TicksQueued" displayName="TicksQueued"/>
		<mapping statName="Ticks/PlayerPawns" displayName="PlayerPawns"/>
        <mapping statName="Ticks/Vehicles" displayName="Vehicles"/>
		<mapping statName="MemoryFreeMB" displayName="MemFreeMB"/>
		<mapping statName="PhysicalUsedMB" displayName="MemUsedMB"/>
		<mapping statName="buildversion" displayName="Build Version"/>
		<mapping statName="config" displayName="Config"/>
		<mapping statName="commandline" displayName="Commandline"/>
		<mapping statName="syncLoads" displayName="Sync Loads"/>
		<mapping statName="scalabilitylevel" displayName="Scaling"/>
		<mapping statName="deviceprofile" displayName="DeviceProfile"/>
		<mapping statName="TargetFPS" displayName="FPS"/>
		<mapping statName="HitchTimePercent" displayName="HitchTime %"/>
		<mapping statName="testname" displayName="Test Name"/>
		<mapping statName="platform" displayName="Platform"/>
		<mapping statName="csvfilename" displayName="CSV Filename"/>
		<mapping statName="rhiname" displayName="RHI Name"/>
		<mapping statName="desktophwinfo" displayName="Hardware"/>
		<mapping statName="extradevelopmentmemorymb" displayName="Extra dev memory (MB)"/>
		<mapping statName="pgoenabled" displayName="PGO"/>
		<mapping statName="fmsglogf/fmsglogfcount" displayName="Logs/frame"/>
		<mapping statName="cameraname" displayName="Camera Name"/>
		<mapping statName="mapname" displayName="Map Name"/>
		<mapping statName="material" displayName="Material"/>
		<mapping statName="mapsequencecomboname" displayName="Map Sequence Combo Name"/>
		<mapping statName="cameracut" displayName="Camera Cut"/>
		<mapping statName="llm" displayName="LLM"/>
		<mapping statName="testbuildispreflight" displayName="Preflight"/>
		<mapping statName="testbuildversion" displayName="Build Version"/>
		<mapping statname="profilegoscenarioname" displayName="ProfileGo Scenario"/>
	</statDisplayNameMappings>

	<derivedMetadataMappings>
		<mapping destName="devicetype" destValue="desktop" metadataQuery="platform=windows or platform=mac or platform=linux"/>
		<mapping destName="devicetype" destValue="mobile" metadataQuery="platform=android or platform=ios"/>
		<!-- Update this or add a mapping if you want a device type for your specific platform-->
		<mapping destName="devicetype" destValue="console" 
			metadataQuery="platform!=android and platform!=ios and platform!=windows and platform!=mac and platform!=linux"/>
		<mapping destName="buildtype" destValue="client" metadataQuery="deviceprofile!=${meta.platform}Server"/>
	</derivedMetadataMappings>

	<!-- Strip Events - these delimit frames which will be excluded from stats computation -->
	<!-- Note: order is important, since these are stripped in the order listed (with no handling for overlapping events) -->
	<csvEventsToStrip>
		<eventPair begin="App_WillDeactivate" end="App_HasReactivated"/>
	</csvEventsToStrip>

	<sharedSummaries>
		<summary refName="psos" type="event" title="PSOs" summaryStatName="PSOs">
			<events>PSO/*</events>
			<colourThresholds>0, 1, 2, 3</colourThresholds>
		</summary>
		<summary refName="pso_framehitches" type="bucketsummary" title="PSO Frame Hitch Correlation" summaryStatName="PSO_Hitches" collapsed="1">
			<stats>PSO/PSOMisses,PSO/PSOMissesOnHitch,PSO/PSOPrevFrameMissesOnHitch,PSO/PSOComputeMisses,PSO/PSOComputeMissesOnHitch,PSO/PSOComputePrevFrameMissesOnHitch</stats>
			<colorDisplay lowEndColor="#00ff00" highEndColor="#ff0000"/>
			<buckets reportAboveRange="1">0, 1, 2, 3, 4, 5, 10, 20, 200</buckets>
		</summary>	
		<summary refName="textureStreaming" type="boundedstatvalues" title="Texture Streaming" useUnstrippedCsvStats="1">
			<column name="% Time textures over budget" formula="percentunderthreshold" threshold="15" metadataKey="texturestreaming_texturesOverPercent" stat="texturestreaming/cachedmips">
				<colourThresholds>0,5,20,100</colourThresholds>
			</column>
			<column name="Cached mips (avg)" formula="average" threshold="15" metadataKey="textureStreaming_cachedMipsAvg" stat="texturestreaming/cachedmips">
				<colourThresholds>300,200,100,50</colourThresholds>
			</column>
			<column name="Streaming pool (avg)" formula="average" threshold="15" metadataKey="textureStreaming_streamingPoolAvg" stat="texturestreaming/streamingpool"/>
			<column name="Wanted mips (avg)" formula="average" threshold="15" metadataKey="textureStreaming_wantedMipsAvg" stat="texturestreaming/wantedmips"/>
			<column name="Temporary pool" formula="average" threshold="15" stat="texturestreaming/temporarypool"/>
			<column name="Safety pool" formula="average" threshold="15" stat="texturestreaming/safetypool"/>
		</summary>	

		<summary refName="ObjectActorCounts" type="peak" title="Object And Actor Counts" >
			<stats>ObjectCount/*, ActorCount/*</stats>
		</summary>

		<!-- I/O uses a boundedstatvalues summary to transform the raw telemetry into more directly actionable stats -->
		<summary refName="io" type="boundedstatvalues" title="I/O (whole run)" useUnstrippedCsvStats="0">
			<column name="% Time Queue Busy" formula="percentoverthreshold" threshold="0" summaryStatName="io_busyPercent" stat="fileio/existingqueuedpackagesqueuedepth">
				<colourThresholds>10,20,30,100</colourThresholds>
			</column>
			<column name="% Time Queue Very Busy" formula="percentoverthreshold" threshold="250" summaryStatName="io_veryBusyPercent" stat="fileio/existingqueuedpackagesqueuedepth">
				<colourThresholds>5,10,15,50</colourThresholds>
			</column>
			<column name="MB/Sec (busy)" formula="sum" multiplier="0.0009765625" filterOutZeros="1" perSecond="1" summaryStatName="io_avgMbPerSecBusy" stat="FileIO/PerFrameKB">
				<colourThresholds>40,20,10,5</colourThresholds>
			</column>
			<column name="MB/Sec" formula="sum" multiplier="0.0009765625" filterOutZeros="0" perSecond="1" summaryStatName="io_avgMbPerSec" stat="FileIO/PerFrameKB"/>
			<column name="Total Loaded (MB)" formula="sum" multiplier="0.0009765625" summaryStatName="io_totalMB" stat="FileIO/PerFrameKB"/>
			<column name="Package depth (avg)" formula="average" summaryStatName="io_avgPackageDepth" stat="fileio/existingqueuedpackagesqueuedepth"/>
			<column name="Package depth busy (avg)" formula="average" filterOutZeros="1" summaryStatName="io_avgPackageDepthBusy" stat="fileio/existingqueuedpackagesqueuedepth"/>
		</summary>

	</sharedSummaries>

	<!-- Preset summary tables -->
	<summaryTableColumnFormatInfo>
		<columnInfo name="count" autoColorize="lowIsBad"/>
        <columnInfo name="framecount" autoColorize="lowIsBad"/>
		<columnInfo name="DynamicResolutionPercentage*" autoColorize="lowIsBad"/>
		<columnInfo name="MemoryFreeMB*" autoColorize="lowIsBad"/>
		<columnInfo name="Total Time*" autoColorize="lowIsBad"/>
		<columnInfo name="pgoenabled" autoColorize="lowIsBad"/>
	</summaryTableColumnFormatInfo>

	<!-- Preset summary tables -->
	<!-- In bulk mode (-csvdir), summary table row data is produced for each CSV. This includes data from all active summaries -->
	<!-- The preset Summary Tables below can be used to provide a filtered and sorted view of that data (see -summaryTable). -->
	<summaryTables>
		<summaryTable name="all" rowSort="buildversion,Platform,deviceprofile">
			<filter>buildversion,Platform,DeviceProfile,scalabilitylevel,*</filter>
		</summaryTable>

		<summaryTable name="default" rowSort="buildversion,profilegoscenarioname,Platform,deviceprofile,scalabilitylevel,TargetFPS">
			<filter>Platform,DeviceProfile,scalabilitylevel,Report,Insights,Video,Horde Job,profilegoscenarioname,TargetFPS,MVP*,Hitches/Min,HitchTimePercent,FrameTime*,GamethreadTime*,Renderthreadtime*,RHIThreadTime*,GPUTime*,MemoryFreeMB Min,PhysicalUsedMB Max,DynamicResolution*,RHI/Drawcalls*,Basic/TicksQueued*,Total Time*,Csv File,buildversion,config,framecount*,syncloads</filter>
		</summaryTable>

		<!-- APT summary tables -->			
		<summaryTable name="sequence" rowSort="buildversion,Platform,deviceprofile,mapsequencecomboname,cameracut,scalabilitylevel,TargetFPS">
			<filter>Platform,DeviceProfile,scalabilitylevel,Report,mapsequencecomboname,cameracut,Horde Job,TargetFPS,MVP*,Hitches/Min,HitchTimePercent,FrameTime*,GamethreadTime*,Renderthreadtime*,RHIThreadTime*,GPUTime*,MemoryFreeMB Min,PhysicalUsedMB Max,DynamicResolution*,RHI/Drawcalls*,Basic/TicksQueued*,Total Time*,Csv File,buildversion,config,playlistid,framecount*,syncloads</filter>
		</summaryTable>

		<summaryTable name="keystats" rowSort="buildversion,Platform">
			<filter>Platform,MVP*,Hitches/Min,HitchTimePercent,FrameTime*,GamethreadTime*,DynamicResolution*,Renderthreadtime*,GPUTime*,PhysicalUsedMB Max</filter>
		</summaryTable>

		<summaryTable name="staticcamera" rowSort="buildversion,Platform,mapname,cameraname,deviceprofile,scalabilitylevel,TargetFPS">
			<filter>Platform,DeviceProfile,mapname,cameraname,scalabilitylevel,Report,Horde Job,TargetFPS,MVP*,Hitches/Min,HitchTimePercent,FrameTime*,GamethreadTime*,Renderthreadtime*,RHIThreadTime*,GPUTime*,MemoryFreeMB Min,PhysicalUsedMB Max,DynamicResolution*,RHI/Drawcalls*,Basic/TicksQueued*,Total Time*,Csv File,buildversion,config,playlistid,framecount*,syncloads</filter>
		</summaryTable>
		
		<summaryTable name="materials" rowSort="buildversion,Platform,material,deviceprofile,scalabilitylevel,TargetFPS">
			<filter>Platform,DeviceProfile,material,scalabilitylevel,Report,Horde Job,TargetFPS,MVP*,Hitches/Min,HitchTimePercent,FrameTime*,GamethreadTime*,Renderthreadtime*,RHIThreadTime*,GPUTime*,MemoryFreeMB Min,PhysicalUsedMB Max,DynamicResolution*,RHI/Drawcalls*,Basic/TicksQueued*,Total Time*,Csv File,buildversion,config,playlistid,framecount*,syncloads</filter>
		</summaryTable>

		<summaryTable name="memory" rowSort="buildversion,Platform,deviceprofile,scalabilitylevel">
			<filter>MemoryFreeMB*,PhysicalUsedMB*,Total Time*,extradevelopmentmemorymb</filter>
		</summaryTable>

		<summaryTable name="condensed" rowSort="buildversion,Platform,deviceprofile,scalabilitylevel,TargetFPS">
			<filter>DeviceProfile,scalabilitylevel,TargetFPS,MVP*,Hitches/Min,MemoryFreeMB Min,PhysicalUsedMB Max,GamethreadTime*,Renderthreadtime*,RHIThreadTime*,GPUTime*,DynamicResolution*,SyncLoads*</filter>
		</summaryTable>

		<summaryTable name="historical" rowSort="buildversion,platform,deviceprofile">
			<filter>buildversion,Platform,DeviceProfile,scalabilitylevel, config,TargetFPS,testname,Report,MVP*,Hitches/Min,HitchTimePercent,FrameTime*,GamethreadTime*,Renderthreadtime*,RHIThreadTime*,GPUTime*,MemoryFreeMB Min,PhysicalUsedMB Max,extradevelopmentmemorymb,DynamicResolution*,RHI/Drawcalls*,Ticks/PlayerPawns*,Basic/TicksQueued*,Total Time*,framecount*,fmsglogf/fmsglogfcount avg,syncloads</filter>
		</summaryTable>

		<summaryTable name="autoPerfReportStandard" rowSort="buildversion,Platform,deviceprofile,profilegoscenarioname" scrollableFormatting="1" reverseSortRows="1">
			<filter>buildversion,DeviceProfile,replayid,profilegoscenarioname,pgoenabled,config,count,testname,TargetFPS,MVP*,Hitches/Min,HitchTimePercent,FrameTime*,GamethreadTime*,Renderthreadtime*,RHIThreadTime*,GPUTime*,MemoryFreeMB Min,PhysicalUsedMB Max,extradevelopmentmemorymb,DynamicResolution*,Total Time*,RHI/*,Basic/TicksQueued*,Ticks/Vehicles*,framecount*,Csv File,Report</filter>
		</summaryTable>

		<summaryTable name="autoPerfReportLlm" rowSort="buildversion" weightByColumn="total time (s)" scrollableFormatting="1" colorizeMode="auto" reverseSortRows="1">
			<filter>buildversion,DeviceProfile,replayid,profilegoscenarioname,Report,Log Report,Csv File,Horde Job,Total Avg,TrackedTotal Avg,Untracked Avg,Textures Avg,Audio Avg,Meshes Avg,RenderTargets Avg,UObject Avg,Animation Avg,Shaders Avg,Filesystem Avg,Physics Avg</filter>
		</summaryTable>

	</summaryTables>

		<!-- Global variable sets - these can be referenced from Report Types and Summaries -->
	<globalVariableSet>
	    <!-- Example of variable sets based on Metadata. This can be used to define
		     and override variable sets based on metadata (For eg. "deviceType=linux")  -->
		<variableSet metadataQuery="buildtype=client">  
			<var name="TargetFrameRate">${meta.targetframerate}</var>

			<!-- Default Budgets-->
			<var name="MvpBudget">3.5</var>
			<var name="DrawCallsBudget">1000</var> 
			<var name="HitchThreshold">100</var>
		
			<!-- Default ${TargetFrameRate} to 60 if not specified -->
			<variableSet metadataQuery="targetframerate=0">			
				<var name="TargetFrameRate">60</var>
			</variableSet>

			<!-- Default budgets (may need overriding) -->
			<var name="HitchesPerMinBudget">1.6</var> 
			<var name="HitchTimePercentBudget">0.3</var>

			<!-- Platform independent budgets -->
			<var name="TicksQueuedBudgets">150, 200, 250, 300</var>
			<var name="TicksBudget">200</var>
			<var name="CameraSpeedBudget">2000</var>
			<var name="MemFreeBudget">300</var>
			<var name="FrameTimeBudgets"></var> <!-- Can be overridden but leave blank to allow it to be auto-generated from MVP -->
			<var name="FrameTimeMultiplier">1</var> <!-- This gives the ratio of frame time relative to a 60hz frame. E.g 30fps = 2. Overridden later -->

			<!-- Default GPU budgets (may need overriding) -->
			<var name="GPUPostprocessingBudget">1.00</var>
			<var name="GPULightsBudget">1.00</var>
			<var name="GPUBasepassBudget">3.00</var>
			<var name="GPUShadowDepthsBudget">1.00</var>
			<var name="GPUShadowProjectionBudget">0.75</var>
			<var name="GPUVirtualTextureBudget">1.0</var>
			<var name="GPUTranslucencyBudget">1.0</var>
			<var name="LightCountAllBudget">20</var>
			<var name="LightCountBatchedBudget">13</var>
			<var name="LightCountUnbatchedBudget">7</var>

			<!-- 30fps or lower -->
			<variableSet metadataQuery="targetframerate&lt;=30">
				<var name="MvpBudget">3.5</var>
				<var name="HitchThreshold">100</var>

				<variableSet metadataQuery="targetframerate=20">
					<var name="FrameTimeMultiplier">3</var>
					<var name="BudgetMultiplier">1.5</var>
					<var name="DrawCallsBudget">600</var> 
				</variableSet>
			 
				<variableSet metadataQuery="targetframerate=30">
					<var name="FrameTimeMultiplier">2</var>
					<var name="BudgetMultiplier">1.0</var>
					<var name="DrawCallsBudget">860</var> 
				</variableSet>		
			
				<variableSet multiplier="${BudgetMultiplier}">
					<var name="GameThreadTimeBudget">19.0</var> 
					<var name="RenderThreadTimeBudget">16.7</var> 
					<var name="RHIThreadTimeBudget">12.0</var> 
					<var name="GPUTimeBudget">30</var> <!-- Assumes no dynamic res. Will be cleared if we have dynamic res  --> 
				</variableSet>

			</variableSet>

			<!-- Higher than 30fps -->
			<variableSet metadataQuery="targetframerate&gt;30 or targetframerate=0">
				<variableSet metadataQuery="targetframerate=45">
					<var name="FrameTimeMultiplier">1.33</var>
					<var name="HitchThreshold">80</var>
				</variableSet>

				<variableSet metadataQuery="targetframerate=60 or targetframerate=0">
					<var name="FrameTimeMultiplier">1</var>
					<var name="HitchThreshold">60</var>
				</variableSet>

				<variableSet metadataQuery="targetframerate=90">
					<var name="FrameTimeMultiplier">0.66</var>
					<var name="HitchThreshold">40</var>
				</variableSet>
			
				<variableSet metadataQuery="targetframerate=120">
					<var name="FrameTimeMultiplier">0.5</var>
					<var name="HitchThreshold">30</var>
				</variableSet>
				
				<variableSet multiplier="${FrameTimeMultiplier}">
					<var name="GameThreadTimeBudget">15.00</var> 
					<var name="RenderThreadTimeBudget">13.0</var> 
					<var name="RHIThreadTimeBudget">7.5</var> 
					<var name="GPUTimeBudget">15</var> 
				</variableSet>	
			</variableSet>

			<var name="IdealFrameTime"			multiplier="${FrameTimeMultiplier}"	>16.666666</var>

			<!-- Set color thresholds for the main stats (green/yellow/orange/red), based on the main budgets -->
			<var name="MvpBudgets"				multiplier="${MvpBudget}"				>1.00, 1.42, 2.05, 7.65</var> 
			<var name="GameThreadTimeBudgets"	multiplier="${GameThreadTimeBudget}"	>1.00, 1.05, 1.11, 1.72</var> 
			<var name="RenderThreadTimeBudgets" multiplier="${RenderThreadTimeBudget}"	>1.00, 1.05, 1.11, 1.35</var> 
			<var name="RHIThreadTimeBudgets"	multiplier="${RHIThreadTimeBudget}"		>1.00, 1.05, 1.11, 1.46</var> 
			<var name="DrawCallsBudgets"		multiplier="${DrawCallsBudget}"			>1.00, 1.03, 1.08, 1.57</var> 
			<var name="GPUTimeBudgets"			multiplier="${GPUTimeBudget}"			>1.00, 1.05, 1.11, 1.30</var> 
			<var name="HitchesPerMinBudgets"	multiplier="${HitchesPerMinBudget}"		>1.00, 1.10, 1.24, 11.8</var>
			<var name="HitchTimePercentBudgets"	multiplier="${HitchTimePercentBudget}"	>1.00, 1.40, 2.00, 8.13</var>

			<!-- Detailed report summary variables -->
			<var name="HitchSummaryCountThresholds">33, 20, 16.6, 16.2</var>
			<var name="HitchSummaryTimeBuckets">${HitchThreshold}, 150, 250, 500, 750, 1000, 2000</var> 
			<variableSet metadataQuery="targetframerate&gt;60">
				<var name="HitchSummaryTimeBuckets">${HitchThreshold}, 60, 120, 250, 500, 750, 1000</var> 
			</variableSet>	
			<var name="HistogramThresholds" multiplier="${IdealFrameTime}">1.00, 1.50, 2.00, 3.60</var> 
			<var name="MvpStatName">MVP${TargetFrameRate}</var>
			<!-- Stat mapping for budget colored heatmaps, if a Stat has a var named StatBudgets, it will automap (like RenderThreadTime:RenderThreadTimeBudgets) -->
			<var name="HeatmapBudgetStats">DynamicResolutionPercentage:DynamicResBudgets, MemoryFreeMB:MemoryFreeMinBudgets, RHI/DrawCalls:DrawCallsBudgets, GPU/Total:GPUTimeBudgets</var>
		
			<!-- Compute TextureStreamingMaxY for the texture streaming graph based on poolsize metadata (if it exists) -->
			<var name="TextureStreamingMaxY">-1</var> 
			<variableSet metadataQuery="streamingpoolsizemb&gt;=0">
				<var name="TextureStreamingMaxY" multiplier="1.05">${meta.streamingpoolsizemb}</var>
			</variableSet>

			<!-- Default Memory budgets -->
			<var name="UObjectBudget">600</var>
			<var name="TexturesBudget">995</var>
			<var name="AudioBudget">120</var>
			<var name="MeshesBudget">270</var>
			<var name="RenderTargetsBudget">290</var>
			<var name="AnimationBudget">77</var>
			<var name="ShadersBudget">160</var>
			<var name="FileSystemBudget">150</var>
			<var name="PhysicsBudget">86</var>
			<var name="ProgramSizeBudget">340</var>
			<var name="TrackedTotalBudget">11500</var>
			<var name="UnTrackedTotalBudget">780</var>
			<var name="IASBudget">16</var>
			<var name="UIBudget">26</var>
			<var name="EngineMiscBudget">55</var>
			<var name="RealTimeCommunicationBudget">14</var>
			<var name="FMallocUnusedBudget">570</var>

		</variableSet>
	</globalVariableSet>

	<reporttypes reportGraphsFile="ReportGraphs.xml">
		<reporttype name="ClientPerf" title="Client ${TargetFrameRate}FPS Performance Report" vsync="1">
			<autodetection requiredstats="frametime">
				<requiredmetadata key="targetframerate" allowedValues="60" stopIfKeyFound="1" ignoreIfKeyNotFound="1"/>
			</autodetection>
			
			<metadataToShow>buildversion,platform,deviceprofile,profilegoscenarioname,csvfilename,rhiname,desktophwinfo,extradevelopmentmemorymb,testname,replayid,mapsequencecomboname</metadataToShow>

			<summary type="fpschart" fps="60" hitchThreshold="60" useEngineHitchMetric="1">
				<stats>Frametime,GameThreadtime,RenderThreadtime,RHIThreadTime,GPUtime,RHI/Drawcalls,DynamicResolutionPercentage,MemoryFreeMB(min),PhysicalUsedMB(max),Basic/TicksQueued,fmsglogf/fmsglogfcount</stats>
				<colourThresholds stat="Hitches/Min"> 2, 3, 5, 6</colourThresholds>
				<colourThresholds stat="HitchTimePercent"> 0.1, 0.5, 1, 2</colourThresholds>
				<colourThresholds stat="MVP60"> 2, 3, 4, 5</colourThresholds>
				<colourThresholds stat="MemoryFreeMB(min)">400,300,200,50</colourThresholds>
				<colourThresholds stat="DynamicResolutionPercentage">90,80,70,60</colourThresholds>
				<colourThresholds stat="Frametime"> 16.7, 17.5, 19, 22.5 </colourThresholds>
				<colourThresholds stat="GameThreadtime"> 16.7, 17.5, 19, 22.5 </colourThresholds>
				<colourThresholds stat="RHIThreadTime"> 16.7, 17.5, 19, 22.5 </colourThresholds>
				<colourThresholds stat="RenderThreadtime"> 16.7, 17.5, 19, 22.5 </colourThresholds>
				<colourThresholds stat="GPUtime"> 16.7, 17.5, 19, 22.5 </colourThresholds>
				<colourThresholds stat="Basic/TicksQueued"> 150, 200, 250, 300 </colourThresholds>
				<colourThresholds stat="fmsglogf/fmsglogfcount">0.01,0.1,0.2,0.5</colourThresholds>
			</summary>

			<summary type="histogram" suppressAveragesTable="1">
				<stats>Frametime,GameThreadtime,RenderThreadtime,RHIThreadTime,GPUtime</stats>
				<histogramThresholds>${HistogramThresholds}</histogramThresholds>
			</summary>
						
			<summaryRef name="psos"/>
		  	<summaryRef name="pso_framehitches"/>
			<summaryRef name="io"/>
			<summaryRef name="textureStreaming"/>
			<summaryRef name="ObjectActorCounts"/>

			<summary type="hitches" collapsed="1">
				<stats>Frametime,GameThreadtime,RenderThreadtime,RHIThreadTime,GPUtime</stats>
				<colourThresholds>${HitchSummaryCountThresholds}</colourThresholds>
				<hitchThresholds>${HitchSummaryTimeBuckets}</hitchThresholds>
			</summary>

			<graph title="Stat Unit" budget="16.66" />
			<graph title="Stat Unit Raw" budget="16.66" />
			<graph title="Gamethread Breakdown" budget="16.66"/>
			<graph title="Gamethread Breakdown Raw" budget="16.66"/>
			<graph title="Gamethread Waits" budget="2"/>
			<graph title="Gamethread Physics" budget="2"/>
			<graph title="Gamethread Breakdown Giant" budget="16.66" external="1"/>

			<graph title="Ticks Breakdown" budget="200"/>
			<graph title="File I/O"/>
			<graph title="Actor Counts" budget="0"/>
			<graph title="Object Count" />

			<graph title="Dynamic Resolution" budget="80"/>
			<graph title="GPU Breakdown" budget="16.66" />
			<graph title="GPU Breakdown Raw" budget="16.66" />
			<graph title="D3D12 Video Memory"  budget="0" />
			<graph title="Niagara Breakdown" budget="33.33"/>
			<graph title="Light Counts" budget="0" />
			<graph title="RHI Drawcalls" budget="3000" />
			<graph title="Renderthread Breakdown" budget="${RenderThreadTimeBudget}"/>
			<graph title="Renderthread Breakdown Raw" budget="${RenderThreadTimeBudget}"/>
			<graph title="RHI Drawcalls" budget="${DrawCallsBudget}" />
			<graph title="RHI Drawcalls breakdown" budget="${DrawCallsBudget}"/>
			<graph title="FScene Breakdown" budget="33.33"/>
			<graph title="FScene Breakdown Raw" budget="33.33"/>
			<graph title="Render Targets Peak(MB)" budget="0" />
			<graph title="Textures Peak(MB)" budget="0" />
			<graph title="Virtual Texture" budget="0" />
			<graph title="Animation Budget" budget="1.5" />
			<graph title="Texture Streaming" budget="0" />

			<graph title="Physical Memory" budget="${MemFreeBudget}"/>
			<graph title="Physical Memory Over Budget" budget="${MemFreeBudget}"/>
			<graph title="MallocBinned Total Bin Mem (MB)"/>
			<graph title="MallocBinned Bins Fragmentation (%)"/>
			<graph title="MallocBinned Bins Waste (MB)"/>
			<graph title="MallocBinned Allocations Breakdown"/>
			<graph title="RenderTarget Pool"/>
			<graph title="Nanite Streaming"/>

			<graph title="Log Counts" budget="0" />
			<graph title="CSV Profiler" budget="0" />
		</reporttype>

		<!-- 60fps report -->
		<reporttype name="Default60fps" title="60FPS Performance Report" vsync="1">
			<autodetection requiredstats="frametime">
				<requiredmetadata key="targetframerate" allowedValues="60" stopIfKeyFound="1" ignoreIfKeyNotFound="1"/>
			</autodetection>
			
			<metadataToShow>buildversion,platform,deviceprofile,csvfilename,rhiname,desktophwinfo,extradevelopmentmemorymb,testname</metadataToShow>

			<summary type="fpschart" fps="60" hitchThreshold="60" useEngineHitchMetric="1">
				<stats>Frametime,GameThreadtime,RenderThreadtime,RHIThreadTime,GPUtime,RHI/Drawcalls,DynamicResolutionPercentage,MemoryFreeMB(min),PhysicalUsedMB(max),Basic/TicksQueued,fmsglogf/fmsglogfcount</stats>
				<colourThresholds stat="Hitches/Min"> 2, 3, 5, 6</colourThresholds>
				<colourThresholds stat="HitchTimePercent"> 0.1, 0.5, 1, 2</colourThresholds>
				<colourThresholds stat="MVP60"> 2, 3, 4, 5</colourThresholds>
				<colourThresholds stat="MemoryFreeMB(min)">400,300,200,50</colourThresholds>
				<colourThresholds stat="DynamicResolutionPercentage">90,80,70,60</colourThresholds>
				<colourThresholds stat="Frametime"> 16.7, 17.5, 19, 22.5 </colourThresholds>
				<colourThresholds stat="GameThreadtime"> 16.7, 17.5, 19, 22.5 </colourThresholds>
				<colourThresholds stat="RHIThreadTime"> 16.7, 17.5, 19, 22.5 </colourThresholds>
				<colourThresholds stat="RenderThreadtime"> 16.7, 17.5, 19, 22.5 </colourThresholds>
				<colourThresholds stat="GPUtime"> 16.7, 17.5, 19, 22.5 </colourThresholds>
				<colourThresholds stat="Basic/TicksQueued"> 150, 200, 250, 300 </colourThresholds>
				<colourThresholds stat="fmsglogf/fmsglogfcount">0.01,0.1,0.2,0.5</colourThresholds>
			</summary>

			<summary type="hitches">
				<stats>Frametime,GameThreadtime,RenderThreadtime,RHIThreadTime,GPUtime</stats>
				<colourThresholds>33, 20, 16.6, 16.2</colourThresholds>
				<hitchThresholds>60, 150, 250, 500, 750, 1000, 2000</hitchThresholds>
			</summary>

			<summaryRef name="io"/>

			<graph title="Stat Unit" budget="16.66" />
			<graph title="Stat Unit Raw" budget="16.66" />
			<graph title="Gamethread Breakdown" budget="16.66"/>
			<graph title="Gamethread Breakdown Raw" budget="16.66"/>
			<graph title="Gamethread Waits" budget="2"/>
			<graph title="Gamethread Physics" budget="2"/>
			<graph title="Gamethread Breakdown Giant" budget="16.66" external="1"/>
			<graph title="Ticks Breakdown" budget="200"/>
			<graph title="File I/O"/>
			<graph title="Actor Counts" budget="0"/>
			<graph title="Cmds DataLayers" budget="300"/>
			<graph title="Physical Memory" budget="300"/>
			<graph title="Physical Memory Over Budget" budget="300"/>
			<graph title="Dynamic Resolution" budget="80"/>
			<graph title="GPU Breakdown" budget="16.66" />
			<graph title="GPU Breakdown Raw" budget="16.66" />
			<graph title="D3D12 Video Memory"  budget="0" />
			<graph title="Niagara Breakdown" budget="33.33"/>
			<graph title="Light Counts" budget="0" />
			<graph title="RHI Drawcalls" budget="3000" />
			<graph title="Renderthread Breakdown" budget="16.66"/>
			<graph title="Renderthread Breakdown Raw" budget="16.66"/>
			<graph title="FScene Breakdown" budget="33.33"/>
			<graph title="FScene Breakdown Raw" budget="33.33"/>
			<graph title="Render Targets Peak(MB)" budget="0" />
			<graph title="Textures Peak(MB)" budget="0" />
			<graph title="Virtual Texture" budget="0" />
			<graph title="Animation Budget" budget="1.5" />
			<graph title="Texture Streaming" budget="0" />
			<graph title="Log Counts" budget="0" />
			<graph title="CSV Profiler" budget="0" />
		</reporttype>


		<!-- 30 Fps report -->
		<reporttype name="Default30fps" title="30FPS Performance Report" vsync="1">
			<autodetection requiredstats="frametime">
				<requiredmetadata key="targetframerate" allowedValues="30" stopIfKeyFound="1" ignoreIfKeyNotFound="1"/>
			</autodetection>

			<metadataToShow>buildversion,platform,deviceprofile,csvfilename,rhiname,desktophwinfo,extradevelopmentmemorymb,testname</metadataToShow>

			<summary type="fpschart" fps="30" hitchThreshold="60" useEngineHitchMetric="1">
				<stats>Frametime,GameThreadtime,RenderThreadtime,RHIThreadTime,GPUtime,RHI/Drawcalls,DynamicResolutionPercentage,MemoryFreeMB(min),PhysicalUsedMB(max),Basic/TicksQueued</stats>
				<colourThresholds stat="Hitches/Min"> 0, 2.5, 3.75, 5</colourThresholds>
				<colourThresholds stat="HitchTimePercent"> 0.1, 0.5, 1, 2</colourThresholds>
				<colourThresholds stat="MVP30"> 0, 2, 3, 4</colourThresholds>
				<colourThresholds stat="MemoryFreeMB">400,300,200,100</colourThresholds>
				<colourThresholds stat="MemoryFreeMB(min)">400,300,200,50</colourThresholds>
				<colourThresholds stat="DynamicResolutionPercentage">90,80,70,60</colourThresholds>
				<colourThresholds stat="Frametime"> 33.4, 35, 38, 45  </colourThresholds>
				<colourThresholds stat="GameThreadtime"> 33.4, 35, 38, 45  </colourThresholds>
				<colourThresholds stat="RHIThreadTime"> 33.4, 35, 38, 45  </colourThresholds>
				<colourThresholds stat="RenderThreadtime"> 33.4, 35, 38, 45  </colourThresholds>
				<colourThresholds stat="GPUtime"> 33.4, 35, 38, 45  </colourThresholds>
				<colourThresholds stat="Basic/TicksQueued"> 150, 200, 250, 300 </colourThresholds>
			</summary>

			<summary type="hitches">
				<stats>Frametime,GameThreadtime,RenderThreadtime,RHIThreadTime,GPUtime</stats>
				<colourThresholds> 20, 16.6, 16.2</colourThresholds>
				<hitchThresholds>60, 150, 250, 500, 750, 1000, 2000</hitchThresholds>
			</summary>


			<summaryRef name="io"/>
			<summaryRef name="mapOverlay"/>
			<!--
			<summaryRef name="syncLoads"/>
			<summaryRef name="textureStreaming"/>
			<summaryRef name="psos"/>
			-->

			<graph title="Stat Unit 30hz" budget="33.33" />
			<graph title="Stat Unit Raw" budget="33.33" />
			<graph title="Gamethread Breakdown" budget="33.33"/>
			<graph title="Gamethread Breakdown Raw" budget="33.33"/>
			<graph title="Gamethread Waits" budget="4"/>
			<graph title="Gamethread Physics" budget="4"/>
			<graph title="Gamethread Breakdown Giant" budget="33.33" external="1"/>
			<graph title="Ticks Breakdown" budget="200"/>
			<graph title="File I/O"/>
			<graph title="Actor Counts" budget="0"/>
			<graph title="Physical Memory" budget="300"/>
			<graph title="Cmds DayaLayers" budget="300"/>
			<graph title="Physical Memory Over Budget" budget="300"/>
			<graph title="Dynamic Resolution" budget="80"/>
			<graph title="GPU Breakdown" budget="33.33" />
			<graph title="GPU Breakdown Raw" budget="16.66" />
			<graph title="Niagara Breakdown" budget="33.33"/>
			<graph title="RHI Drawcalls" budget="2000" />
			<graph title="Renderthread Breakdown" budget="33.33"/>
			<graph title="Renderthread Breakdown Raw" budget="33.33"/>
			<graph title="FScene Breakdown" budget="33.33"/>
			<graph title="FScene Breakdown Raw" budget="33.33"/>
			<graph title="Render Targets Peak(MB)" budget="0" />
			<graph title="Textures Peak(MB)" budget="0" />
			<graph title="Virtual Texture" budget="0" />
			<graph title="Light Counts" budget="0" />
			<graph title="Animation Budget" budget="3.0" />
			<graph title="Texture Streaming" budget="0" />
			<graph title="Log Counts" budget="0" />
			<graph title="CSV Profiler" budget="0" />
		</reporttype>


		<!-- Memory report -->
		<reporttype name="Memory" title="Physical Memory Report" vsync="1">
			<autodetection requiredstats="memoryfreemb,physicalusedmb"/>
			<summary type="fpschart" fps="60" hitchThreshold="60" useEngineHitchMetric="1">
				<stats>MemoryFreeMB(min),PhysicalUsedMB(max),MemoryFreeMB,PhysicalUsedMB,PhysicalUsedMB(min)</stats>
				<colourThresholds stat="MemoryFreeMB(min)">400,300,200,50</colourThresholds>
			</summary>
			<graph title="Physical Memory" budget="300"/>
		</reporttype>


		<!-- LLM Report -->
		<reporttype name="LLM" title="LLM ${meta.platform} Report" ignoreList="" stripEvents="0">
			<autodetection requiredstats="llm/total">
				<requiredmetadata key="llm" allowedValues="1"/>
			</autodetection>

			<metadataToShow>buildversion,platform,deviceprofile,csvfilename,rhiname,desktophwinfo,extradevelopmentmemorymb,testname,replayid,mapsequencecomboname</metadataToShow>

			<summary type="peak" hideStatPrefix="llm/">
				<summarySection title="Main">
					<statFilter>LLM/Total,LLM/TrackedTotal,LLM/Untracked,LLM/Textures,LLM/Audio,LLM/Meshes,LLM/RenderTargets,LLM/UObject,LLM/Animation,LLM/Shaders,LLM/Filesystem,LLM/Physics,LLM/ProgramSize</statFilter>
				</summarySection>

				<summarySection title="Audio">
					<statFilter>LLM/Audio/*</statFilter>
				</summarySection>

				<summarySection title="Mesh">
					<statFilter>LLM/Meshes/*</statFilter>
				</summarySection>

				<summarySection title="Physics">
					<statFilter>LLM/Physics/*</statFilter>
				</summarySection>

				<summarySection title="Other">
					<statFilter>LLM/*</statFilter>
				</summarySection>

				<summarySection title="Platform">
					<statFilter>LLMPlatform/Total,LLMPlatform/TrackedTotal,LLMPlatform/*</statFilter>
				</summarySection>

				<summarySection title="Physical">
					<statFilter>PhysicalUsed*,MemoryFree*</statFilter>
				</summarySection>
			</summary>

			<!-- top-level LLM summary graphs -->
			<graph title="Total Memory Stats"			inSummary="1"/>
			<graph title="Physical Memory"              inSummary="1"/>
			<graph title="Actor Counts" />
			<graph title="Actors Spawned" />
			<graph title="Actors Streamed" />
			<graph title="Actors Persistent" />
			<graph title="Object Count" />

			<!-- Memory breakdowns -->
			<graph title="Tracked Memory Breakdown"		inSummary="1"/>
			<graph title="Content Memory Breakdown"/>
			<graph title="Audio Memory Breakdown"		inSummary="1"/>
			<graph title="Mesh Memory Breakdown"		inSummary="1"/>
			<graph title="Physics Memory Breakdown"		inSummary="1"/>

			<!-- Various collections -->
			<graph title="Rendering Memory Stats"		/>
			<graph title="Misc Memory Stats"			/>
			<graph title="Development Memory Stats"		/>
			<graph title="Initialization Memory Stats"  />
			<graph title="Loading Memory Stats"			/>

			<graph title="LLMPlatform"					inSummary="1"/>

			<!-- Individual stats -->
			<graph title="Tracked Total Memory"         budget="${TrackedTotalBudget}" inSummary="1"/>
			<graph title="UObject Memory"               budget="${UObjectBudget}" inSummary="1"/>
			<graph title="Effects Memory"               inSummary="1"/>
			<graph title="Animation Memory"             budget="${AnimationBudget}" inSummary="1"/>
			<graph title="Meshes Memory"                budget="${MeshesBudget}" inSummary="1"/>
			<graph title="Physics Memory"               budget="${PhysicsBudget}" inSummary="1"/>
			<graph title="Audio Memory"                 budget="${AudioBudget}" inSummary="1"/>
			<graph title="EngineMisc Memory"            budget="${EngineMiscBudget}" inSummary="1"/>
			<graph title="Materials Memory"             budget="12" inSummary="1"/>
			<graph title="RenderTargets Memory"         budget="${RenderTargetsBudget}" inSummary="1"/>
			<graph title="UI Memory"                    budget="${UIBudget}" inSummary="1"/>
			<graph title="Shaders Memory"               budget="${ShadersBudget}" inSummary="1"/>
			<graph title="Textures Memory"              budget="${TexturesBudget}" inSummary="1"/>
			<graph title="ProgramSize Memory"           budget="${ProgramSizeBudget}" inSummary="1"/>
			<graph title="FileSystem Memory"            budget="${FileSystemBudget}" inSummary="1"/>

			<graph title="ThreadStack Memory"           budget="15" inSummary="1"/>
			<graph title="FName Memory"                 budget="45" inSummary="1"/>
			<graph title="Networking Memory"            budget="41" inSummary="1"/>
			<graph title="TextureMetaData Memory"       budget="4" inSummary="1"/>
			<graph title="SceneRender Memory"           budget="42" inSummary="1"/>
			<graph title="RHIMisc Memory"               budget="64" inSummary="1"/>
			<graph title="AsyncLoading Memory"          budget="57" inSummary="1"/>
			<graph title="Particles Memory"             budget="9" inSummary="1"/>
			<graph title="Niagara Memory"               budget="9" inSummary="1"/>
			<graph title="GC Memory"                    budget="7" inSummary="1"/>
			<graph title="EnginePreInit Memory"         budget="10" inSummary="1"/>
			<graph title="EngineInit Memory"            budget="15" inSummary="1"/>
			<graph title="LoadMapMisc Memory"           budget="18" inSummary="1"/>

			<graph title="Localization Memory"          budget="41" inSummary="1"/>
			<graph title="AssetRegistry Memory"         budget="49" inSummary="1"/>
			<graph title="ConfigSystem Memory"          budget="16" inSummary="1"/>
			<graph title="InitUObject Memory"           budget="30" inSummary="1"/>
			<graph title="MaterialInstance Memory"      budget="12" inSummary="1"/>
			<graph title="CSVProfiler Memory"           budget="1" inSummary="1"/>
			<graph title="FMallocUnused Memory"         budget="${FMallocUnusedBudget}" inSummary="1"/>
			<graph title="RenderTarget Pool"			budget="0"/>

			<graph title="Untagged Memory"				inSummary="1"  budget="12"/>
			<graph title="Untracked Memory"				inSummary="1"  budget="300"/>
		</reporttype>

		<!-- FX report -->
		<reporttype name="FXReport" title="Playtest FX Perf Report" vsync="1">
			<autodetection requiredstats="frametime">
				<requiredmetadata key="targetframerate" allowedValues="60" stopIfKeyFound="1" ignoreIfKeyNotFound="1"/>
			</autodetection>

			<metadataToShow>testname,buildversion,pgoenabled,deviceprofile,csvfilename,rhiname,desktophwinfo,extradevelopmentmemorymb</metadataToShow>

			<summary type="fpschart" fps="60" hitchThreshold="60" useEngineHitchMetric="1">
				<stats>Frametime,GameThreadtime,RenderThreadtime,GPUtime,exclusive/gamethread/effects</stats>
				<colourThresholds stat="Hitches/Min"> 2, 3, 5, 6</colourThresholds>
				<colourThresholds stat="HitchTimePercent"> 0.1, 0.5, 1, 2</colourThresholds>
				<colourThresholds stat="MVP60"> 2, 3, 4, 5</colourThresholds>
				<colourThresholds stat="MemoryFreeMB(min)">400,300,200,50</colourThresholds>
				<colourThresholds stat="Frametime"> 16.7, 17.5, 19, 22.5 </colourThresholds>
				<colourThresholds stat="GameThreadtime"> 13, 15, 17, 22.5 </colourThresholds>
				<colourThresholds stat="RenderThreadtime"> 13, 15, 17, 22.5 </colourThresholds>
				<colourThresholds stat="GPUtime"> 16.7, 17.5, 19, 22.5 </colourThresholds>
				<colourThresholds stat="exclusive/gamethread/effects"> 1.5, 2, 4, 6 </colourThresholds>			
			</summary>

			<graph title="Gamethread Budget: Effects" budget="0.75"/>
			<graph title="FX Breakdown Count" budget="50" />
			<graph title="FX Breakdown Total" budget="2000.0" />
			<graph title="FX Breakdown GTOnly" budget="1000.0" />			
			<graph title="FX Breakdown RT" budget="1000.0" />	
			<graph title="FX Breakdown GPU" budget="2000.0" />	
			<graph title="FX Breakdown Activation" budget="300.0" />
			<graph title="FX Breakdown Waits" budget="50.0" />
			<graph title="FX Breakdown Per Inst Avg GT" budget="100.0" />
			<graph title="FX Breakdown Per Inst Avg RT" budget="100.0" />
			<graph title="FX Breakdown Per Inst Avg GPU" budget="100.0" />
			<graph title="Niagara Culling" budget="0.0" />
			<graph title="Niagara Culling Breakdown" budget="0.0" />
			<graph title="Niagara Cull Proxy Breakdown" budget="0.0" />
			<graph title="FX Budget Breakdown" budget="1.0" />
			
		</reporttype>

	</reporttypes>
</root>